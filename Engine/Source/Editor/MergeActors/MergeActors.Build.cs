// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class MergeActors : ModuleRules
{
	public MergeActors(ReadOnlyTargetRules Target) : base(Target)
	{
        PrivateIncludePathModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
                "ContentBrowser",
                "Documentation",
                "MeshUtilities",
                "PropertyEditor",
                "WorkspaceMenuStructure",
                "MeshReductionInterface",
                "MeshMergeUtilities",
				"GeometryProcessingInterfaces"
			}
		);

		PrivateDependencyModuleNames.AddRange(
            new string[] {
                "Core",
                "CoreUObject",
                "Engine",
                "InputCore",
                "MaterialUtilities",
                "Slate",
                "SlateCore",
                "EditorFramework",
                "UnrealEd",
				"ToolWidgets"
            }
        );

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
                "ContentBrowser",
                "Documentation",
                "MeshUtilities",
                "MeshMergeUtilities",
                "MeshReductionInterface",
				"GeometryProcessingInterfaces"
            }
		);
	}
}
