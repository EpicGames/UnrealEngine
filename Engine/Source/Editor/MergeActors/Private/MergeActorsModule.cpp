// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Framework/Application/SlateApplication.h"
#include "Textures/SlateIcon.h"
#include "Framework/Docking/TabManager.h"
#include "Styling/AppStyle.h"
#include "IMergeActorsModule.h"
#include "SMergeActorsToolbar.h"
#include "WorkspaceMenuStructure.h"
#include "WorkspaceMenuStructureModule.h"
#include "MeshUtilities.h"
#include "MeshMergingTool/MeshMergingTool.h"
#include "MeshProxyTool/MeshProxyTool.h"
#include "MeshApproximationTool/MeshApproximationTool.h"
#include "Widgets/Docking/SDockTab.h"
#include "IMeshReductionManagerModule.h"
#include "MeshInstancingTool/MeshInstancingTool.h"


#define LOCTEXT_NAMESPACE "MergeActorsModule"


static const FName MergeActorsTabName = FName("MergeActors");

/**
 * Merge Actors module
 */
class FMergeActorsModule : public IMergeActorsModule
{
public:

	/**
	 * Called right after the module DLL has been loaded and the module object has been created
	 */
	virtual void StartupModule() override;

	/**
	 * Called before the module is unloaded, right before the module object is destroyed.
	 */
	virtual void ShutdownModule() override;

	/**
	 * Register an IMergeActorsTool with the module, passing ownership to it
	 */
	virtual bool RegisterMergeActorsTool(TUniquePtr<IMergeActorsTool> Tool) override;

	/**
	 * Unregister an IMergeActorsTool with the module
	 */
	virtual bool UnregisterMergeActorsTool(IMergeActorsTool* Tool) override;

	/**
	 * Get list of registered tools
	 */
	virtual void GetRegisteredMergeActorsTools(TArray<IMergeActorsTool*>& OutTools) override;

private:

	/** Creates the dock tab widget used by the tab manager */
	TSharedRef<SDockTab> CreateMergeActorsTab(const FSpawnTabArgs& Args);

private:

	/** Pointer to the toolbar widget */
	TWeakPtr<SMergeActorsToolbar> MergeActorsToolbarPtr;

	/** List of registered MergeActorsTool instances */
	TArray<TUniquePtr<IMergeActorsTool>> MergeActorsTools;	
};

IMPLEMENT_MODULE(FMergeActorsModule, MergeActors);


TSharedRef<SDockTab> FMergeActorsModule::CreateMergeActorsTab(const FSpawnTabArgs& Args)
{
	// Build array of MergeActorsTool raw pointers
	TArray<IMergeActorsTool*> ToolsToRegister;
	for (const auto& MergeActorTool : MergeActorsTools)
	{
		check(MergeActorTool.Get() != nullptr);
		ToolsToRegister.Add(MergeActorTool.Get());
	}

	// Construct toolbar widget
	TSharedRef<SMergeActorsToolbar> MergeActorsToolbar =
		SNew(SMergeActorsToolbar)
		.ToolsToRegister(ToolsToRegister);

	// Construct dock tab
	TSharedRef<SDockTab> DockTab =
		SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			MergeActorsToolbar
		];

	// Keep weak pointer to toolbar widget
	MergeActorsToolbarPtr = MergeActorsToolbar;

	return DockTab;
}

void FMergeActorsModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(MergeActorsTabName, FOnSpawnTab::CreateRaw(this, &FMergeActorsModule::CreateMergeActorsTab))
		.SetDisplayName(NSLOCTEXT("MergeActorsModule", "TabTitle", "Merge Actors"))
		.SetTooltipText(NSLOCTEXT("MergeActorsModule", "TooltipText", "Open the Merge Actors tab."))
		.SetGroup(WorkspaceMenu::GetMenuStructure().GetToolsCategory())
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "MergeActors.TabIcon"));

	// Register built-in merging tools straight away
	ensure(RegisterMergeActorsTool(MakeUnique<FMeshMergingTool>()));

	
	IMeshReductionManagerModule& MeshReductionModule = FModuleManager::Get().LoadModuleChecked<IMeshReductionManagerModule>("MeshReductionInterface");
	if (MeshReductionModule.GetMeshMergingInterface() != nullptr)
	{
		if (MeshReductionModule.GetMeshMergingInterface()->GetName().Equals(FString("ProxyLODMeshMerging")))
		{
			// Our Native tool
			ensure(RegisterMergeActorsTool(MakeUnique<FMeshProxyTool>()));
		}
	}

	ensure(RegisterMergeActorsTool(MakeUnique<FMeshInstancingTool>()));

	ensure(RegisterMergeActorsTool(MakeUnique<FMeshApproximationTool>()));
}


void FMergeActorsModule::ShutdownModule()
{
	if (FSlateApplication::IsInitialized())
	{
		FGlobalTabmanager::Get()->UnregisterNomadTabSpawner(MergeActorsTabName);		
	}

	MergeActorsTools.Empty();
}


bool FMergeActorsModule::RegisterMergeActorsTool(TUniquePtr<IMergeActorsTool> Tool)
{
	if (Tool.Get() != nullptr && !MergeActorsTools.Contains(MoveTemp(Tool)))
	{
		MergeActorsTools.Add(MoveTemp(Tool));
		
		// If a tool is added while the toolbar widget is active, update it accordingly
		TSharedPtr<SMergeActorsToolbar> MergeActorsToolbar = MergeActorsToolbarPtr.Pin();
		if (MergeActorsToolbar.IsValid())
		{
			MergeActorsToolbar->AddTool(Tool.Get());
		}

		return true;
	}

	return false;
}


bool FMergeActorsModule::UnregisterMergeActorsTool(IMergeActorsTool* Tool)
{
	if (Tool != nullptr)
	{
		if (MergeActorsTools.RemoveAll([=](const TUniquePtr<IMergeActorsTool>& Ptr) { return Ptr.Get() == Tool; }) > 0)
		{
			TSharedPtr<SMergeActorsToolbar> MergeActorsToolbar = MergeActorsToolbarPtr.Pin();
			if (MergeActorsToolbar.IsValid())
			{
				MergeActorsToolbar->RemoveTool(Tool);
			}

			return true;
		}
	}
	return false;
}

void FMergeActorsModule::GetRegisteredMergeActorsTools(TArray<IMergeActorsTool*>& OutTools)
{
	OutTools.Empty();
	for (const auto& MergeActorTool : MergeActorsTools)
	{
		check(MergeActorTool.Get() != nullptr);
		OutTools.Add(MergeActorTool.Get());
	}
}

#undef LOCTEXT_NAMESPACE
