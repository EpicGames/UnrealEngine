// Copyright Epic Games, Inc. All Rights Reserved.

#include "MergeActorsTool.h"
#include "Components/StaticMeshComponent.h"

bool FMergeActorsTool::GetReplaceSourceActors() const
{
	return bReplaceSourceActors;
}

void FMergeActorsTool::SetReplaceSourceActors(bool bInReplaceSourceActors)
{
	bReplaceSourceActors = bInReplaceSourceActors;
}

bool FMergeActorsTool::RunMergeFromSelection()
{
	TArray<TSharedPtr<FMergeComponentData>> SelectionData;
	BuildMergeComponentDataFromSelection(SelectionData, bAllowShapeComponents);

	if (SelectionData.Num() == 0)
	{
		return false;
	}

	FString PackageName;
	if (GetPackageNameForMergeAction(GetDefaultPackageName(), PackageName))
	{
		return RunMerge(PackageName, SelectionData);
	}
	else
	{
		return false;
	}
}

bool FMergeActorsTool::RunMergeFromWidget()
{
	FString PackageName;
	if (GetPackageNameForMergeAction(GetDefaultPackageName(), PackageName))
	{
		return RunMerge(PackageName, GetSelectedComponentsInWidget());
	}
	else
	{
		return false;
	}
}

bool HasAtLeastOneStaticMesh(const TArray<TSharedPtr<FMergeComponentData>>& ComponentsData)
{
	for (const TSharedPtr<FMergeComponentData>& ComponentData : ComponentsData)
	{
		if (!ComponentData->bShouldIncorporate)
			continue;

		const bool bIsMesh = (Cast<UStaticMeshComponent>(ComponentData->PrimComponent.Get()) != nullptr);

		if (bIsMesh)
			return true;
	}

	return false;
}

bool FMergeActorsTool::CanMergeFromSelection() const
{
	TArray<TSharedPtr<FMergeComponentData>> SelectedComponents;
	BuildMergeComponentDataFromSelection(SelectedComponents, bAllowShapeComponents);
	return HasAtLeastOneStaticMesh(SelectedComponents);
}

bool FMergeActorsTool::CanMergeFromWidget() const
{
	const TArray<TSharedPtr<FMergeComponentData>>& SelectedComponents = GetSelectedComponentsInWidget();
	return HasAtLeastOneStaticMesh(SelectedComponents);
}