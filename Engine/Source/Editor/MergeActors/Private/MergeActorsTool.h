// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "IMergeActorsTool.h"
#include "MergeProxyUtils/Utils.h"

/**
 * Merge Actors tool base class
 */
class FMergeActorsTool : public IMergeActorsTool
{
public:
	/** IMergeActorsTool partial implementation */
	bool GetReplaceSourceActors() const override;

	void SetReplaceSourceActors(bool bReplaceSourceActors) override;

	bool RunMergeFromSelection() override;

	bool RunMergeFromWidget() override;

	bool CanMergeFromSelection() const override;

	bool CanMergeFromWidget() const override;

protected:
	/**
	 * Perform the tool merge operation
	 */
	virtual bool RunMerge(const FString& PackageName, const TArray<TSharedPtr<FMergeComponentData>>& SelectedComponents) = 0;

	/**
	 * Returns the selected components from the tool's widget
	 */
	virtual const TArray<TSharedPtr<FMergeComponentData>>& GetSelectedComponentsInWidget() const = 0;

protected:
	bool bReplaceSourceActors = false;
	bool bAllowShapeComponents = true;
};