// Copyright Epic Games, Inc. All Rights Reserved.

#include "MergeProxyUtils/SMeshProxyCommonDialog.h"

#include "Styling/AppStyle.h"
#include "PropertyEditorModule.h"
#include "IDetailChildrenBuilder.h"
#include "Modules/ModuleManager.h"
#include "SlateOptMacros.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Input/SCheckBox.h"
#include "Engine/Selection.h"
#include "Editor.h"

#include "IDetailsView.h"

// TODO: review this, have new namespace? text is same as SMeshInstancingDialog
#define LOCTEXT_NAMESPACE "SMeshMergingDialog"

//////////////////////////////////////////////////////////////////////////
// SMeshProxyCommonDialog

SMeshProxyCommonDialog::SMeshProxyCommonDialog()
{
	bRefreshListView = false;
	MergeStaticMeshComponentsLabel = LOCTEXT("MergeStaticMeshComponentsLabel", "Mesh Components to be incorporated in the merge:");
	SelectedComponentsListBoxToolTip = LOCTEXT("SelectedComponentsListBoxToolTip", "The selected mesh components will be incorporated into the merged mesh");
	DeleteUndoLabel = LOCTEXT("DeleteUndo", "Insufficient mesh components found for merging.");
}

SMeshProxyCommonDialog::~SMeshProxyCommonDialog()
{
	// Remove all delegates
	USelection::SelectionChangedEvent.RemoveAll(this);
	USelection::SelectObjectEvent.RemoveAll(this);
	FEditorDelegates::MapChange.RemoveAll(this);
	FEditorDelegates::NewCurrentLevel.RemoveAll(this);
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMeshProxyCommonDialog::Construct(const FArguments& InArgs)
{
	UpdateSelectedStaticMeshComponents();
	CreateSettingsView();

	// Create widget layout
	this->ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		[
			SNew(SHorizontalBox)
			// Left-side: the component selection pane
			+ SHorizontalBox::Slot()
			.AutoWidth()
			[
				SNew(SBorder)
				// We want to emulate the style of the table we'll place here
				.BorderImage(&FAppStyle::GetWidgetStyle<FTableRowStyle>("TableView.Row").EvenRowBackgroundBrush)
				[
					SNew(SVerticalBox)
					// Selected components count
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
					[
						SNew(SHorizontalBox)
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						[
							SNew(STextBlock)
							.Text(MergeStaticMeshComponentsLabel)
						]
						+ SHorizontalBox::Slot()
						.AutoWidth()
						.VAlign(VAlign_Center)
						.HAlign(HAlign_Left)
						.Padding(FAppStyle::GetMargin("StandardDialog.SlotPadding"))
						[
							SNew(STextBlock)
							.Text_Lambda([this]() -> FText
							{
								return FText::AsNumber(ComponentSelectionControl.NumSelectedMeshComponents);
							})
						]
					]
					// List of selected/unselected components
					+ SVerticalBox::Slot()
					.AutoHeight()
					.Padding(FAppStyle::GetMargin("StandardDialog.ContentPadding") + FAppStyle::GetMargin("StandardDialog.SlotPadding"))
					[
						SAssignNew(ComponentSelectionControl.ComponentsListView, SListView<TSharedPtr<FMergeComponentData>>)
						.ListItemsSource(&ComponentSelectionControl.SelectedComponents)
						.OnGenerateRow(this, &SMeshProxyCommonDialog::MakeComponentListItemWidget)
						.ToolTipText(SelectedComponentsListBoxToolTip)
					]
				]
			]
			// Right-side: the settings view
			+ SHorizontalBox::Slot()
			[
				SNew(SBorder)
				.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
				[
					SettingsView->AsShared()
				]
			]
		]

		// Diagnostics bar
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10)
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor::Yellow)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Visibility_Lambda([this]()->EVisibility { return this->GetContentEnabledState() ? EVisibility::Collapsed : EVisibility::Visible; })
			[
				SNew(STextBlock)
				.Text(DeleteUndoLabel)
			]
		]

		// Predictive results bar
		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(10)
		[
			SNew(SBorder)
			.BorderBackgroundColor(FLinearColor::Green)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Visibility_Lambda([this]()->EVisibility { return this->GetPredictedResultsText().IsEmpty() ? EVisibility::Collapsed : EVisibility::Visible; })
			[
				SNew(STextBlock)
				.Text(this, &SMeshProxyCommonDialog::GetPredictedResultsText)
			]
		]
	];

	// Selection change
	USelection::SelectionChangedEvent.AddRaw(this, &SMeshProxyCommonDialog::OnLevelSelectionChanged);
	USelection::SelectObjectEvent.AddRaw(this, &SMeshProxyCommonDialog::OnLevelSelectionChanged);
	FEditorDelegates::MapChange.AddSP(this, &SMeshProxyCommonDialog::OnMapChange);
	FEditorDelegates::NewCurrentLevel.AddSP(this, &SMeshProxyCommonDialog::OnNewCurrentLevel);
}

void SMeshProxyCommonDialog::OnMapChange(uint32 MapFlags)
{
	Reset();
}

void SMeshProxyCommonDialog::OnNewCurrentLevel()
{
	Reset();
}

void SMeshProxyCommonDialog::Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime)
{
	// Check if we need to update selected components and the listbox
	if(bRefreshListView == true)
	{
		ComponentSelectionControl.UpdateSelectedCompnentsAndListBox();
		bRefreshListView = false;
	}
}

void SMeshProxyCommonDialog::Reset()
{
	bRefreshListView = true;
}

bool SMeshProxyCommonDialog::GetContentEnabledState() const
{
	return (GetNumSelectedMeshComponents() >= 1); // Only enabled if a mesh is selected
}

void SMeshProxyCommonDialog::UpdateSelectedStaticMeshComponents()
{
	ComponentSelectionControl.UpdateSelectedStaticMeshComponents();
}

TSharedRef<ITableRow> SMeshProxyCommonDialog::MakeComponentListItemWidget(TSharedPtr<FMergeComponentData> ComponentData, const TSharedRef<STableViewBase>& OwnerTable)
{
	return ComponentSelectionControl.MakeComponentListItemWidget(ComponentData, OwnerTable);
}


void SMeshProxyCommonDialog::CreateSettingsView()
{
	// Create a property view
	FPropertyEditorModule& EditModule = FModuleManager::Get().GetModuleChecked<FPropertyEditorModule>("PropertyEditor");

	FDetailsViewArgs DetailsViewArgs;
	DetailsViewArgs.bUpdatesFromSelection = true;
	DetailsViewArgs.bLockable = true;
	DetailsViewArgs.bCustomNameAreaLocation = false;
	DetailsViewArgs.NameAreaSettings = FDetailsViewArgs::ENameAreaSettings::HideNameArea;
	DetailsViewArgs.bCustomFilterAreaLocation = true;
	DetailsViewArgs.DefaultsOnlyVisibility = EEditDefaultsOnlyNodeVisibility::Hide;

	SettingsView = EditModule.CreateDetailView(DetailsViewArgs);

	// Tiny hack to hide this setting, since we have no way / value to go off to 
	struct Local
	{
		/** Delegate to show all properties */
		static bool IsPropertyVisible(const FPropertyAndParent& PropertyAndParent, bool bInShouldShowNonEditable)
		{
			return (PropertyAndParent.Property.GetFName() != GET_MEMBER_NAME_CHECKED(FMaterialProxySettings, GutterSpace));
		}
	};
	
	SettingsView->SetIsPropertyVisibleDelegate(FIsPropertyVisible::CreateStatic(&Local::IsPropertyVisible, true));
	
	SettingsView->OnFinishedChangingProperties().AddSP(this, &SMeshProxyCommonDialog::OnSettingChanged);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SMeshProxyCommonDialog::OnLevelSelectionChanged(UObject* Obj)
{
	Reset();
}

void SMeshProxyCommonDialog::OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent)
{
	Reset();
}

FText SMeshProxyCommonDialog::GetPredictedResultsTextInternal() const
{
	return FText();
}

FText SMeshProxyCommonDialog::GetPredictedResultsText() const
{
	return GetPredictedResultsTextInternal();
}

#undef LOCTEXT_NAMESPACE