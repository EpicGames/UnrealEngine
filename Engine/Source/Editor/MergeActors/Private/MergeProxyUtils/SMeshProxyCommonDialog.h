// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

#include "MergeProxyUtils/Utils.h"

class IDetailsView;

/*-----------------------------------------------------------------------------
   SMeshProxyCommonDialog
-----------------------------------------------------------------------------*/
class SMeshProxyCommonDialog : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SMeshProxyCommonDialog)
	{
	}

	SLATE_END_ARGS()

public:
	/** **/
	SMeshProxyCommonDialog();
	~SMeshProxyCommonDialog();

	/** SWidget functions */
	void Construct(const FArguments& InArgs);

	/** Getter functionality */
	const TArray<TSharedPtr<FMergeComponentData>>& GetSelectedComponents() const { return ComponentSelectionControl.SelectedComponents; }
	/** Get number of selected meshes */
	const int32 GetNumSelectedMeshComponents() const { return ComponentSelectionControl.NumSelectedMeshComponents; }

	/** Resets the state of the UI and flags it for refreshing */
	void Reset();

protected:
	/** Predicted results of the merge given the current settings */
	virtual FText GetPredictedResultsTextInternal() const;

private:
	/** Begin override SCompoundWidget */
	virtual void Tick(const FGeometry& AllottedGeometry, const double InCurrentTime, const float InDeltaTime) override;
	/** End override SCompoundWidget */

	/** Creates and sets up the settings view element*/
	void CreateSettingsView();

	/** Delegate for the creation of the list view item's widget */
	TSharedRef<ITableRow> MakeComponentListItemWidget(TSharedPtr<FMergeComponentData> ComponentData, const TSharedRef<STableViewBase>& OwnerTable);

	/** Delegate to determine whether or not the UI elements should be enabled (determined by number of selected actors / mesh components) */
	bool GetContentEnabledState() const;

	/** Editor delegates for map and selection changes */
	void OnLevelSelectionChanged(UObject* Obj);
	void OnMapChange(uint32 MapFlags);
	void OnNewCurrentLevel();
	void OnSettingChanged(const FPropertyChangedEvent& InPropertyChangedEvent);

	/** Delegates for predicted results display */
	FText GetPredictedResultsText() const;

	/** Updates the selection control */
	void UpdateSelectedStaticMeshComponents();

protected:
	FComponentSelectionControl ComponentSelectionControl;

	/** Settings view ui element ptr */
	TSharedPtr<IDetailsView> SettingsView;

	/** List view state tracking data */
	bool bRefreshListView;

	/** Labels for various parts of the dialog */
	FText MergeStaticMeshComponentsLabel;
	FText SelectedComponentsListBoxToolTip;
	FText DeleteUndoLabel;
};