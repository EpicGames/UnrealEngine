// Copyright Epic Games, Inc. All Rights Reserved.

#include "MergeProxyUtils/Utils.h"

#include "Misc/PackageName.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"

#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Components/StaticMeshComponent.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"
#include "Widgets/Input/SCheckBox.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "Components/ChildActorComponent.h"
#include "Components/ShapeComponent.h"

#define LOCTEXT_NAMESPACE "MergeProxyDialog"

void BuildMergeComponentDataFromSelection(TArray<TSharedPtr<FMergeComponentData>>& OutComponentsData, bool bAllowShapeComponents)
{
	OutComponentsData.Empty();

	// Retrieve selected actors
	USelection* SelectedActors = GEditor->GetSelectedActors();

	TSet<AActor*> Actors;

	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			Actors.Add(Actor);

			// Add child actors & actors found under foundations
			Actor->EditorGetUnderlyingActors(Actors);
		}
	}

	for (AActor* Actor : Actors)
	{
		check(Actor != nullptr);

		TArray<UPrimitiveComponent*> PrimComponents;
		Actor->GetComponents(PrimComponents);
		for (UPrimitiveComponent* PrimComponent : PrimComponents)
		{
			bool bInclude = false; // Should put into UI list
			bool bShouldIncorporate = false; // Should default to part of merged mesh
			bool bIsMesh = false;
			if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(PrimComponent))
			{
				bShouldIncorporate = (StaticMeshComponent->GetStaticMesh() != nullptr);
				bInclude = true;
				bIsMesh = true;
			}
			else if (UShapeComponent* ShapeComponent = Cast<UShapeComponent>(PrimComponent))
			{
				if (bAllowShapeComponents)
				{
					bShouldIncorporate = true;
					bInclude = true;
				}
			}

			if (bInclude)
			{
				OutComponentsData.Add(TSharedPtr<FMergeComponentData>(new FMergeComponentData(PrimComponent)));
				TSharedPtr<FMergeComponentData>& ComponentData = OutComponentsData.Last();
				ComponentData->bShouldIncorporate = bShouldIncorporate;
			}
		}
	}
}

void BuildActorsListFromMergeComponentsData(const TArray<TSharedPtr<FMergeComponentData>>& InComponentsData, TArray<AActor*>& OutActors, TArray<ULevel*>* OutLevels /* = nullptr */)
{
	for (const TSharedPtr<FMergeComponentData>& SelectedComponent : InComponentsData)
	{
		if (SelectedComponent->PrimComponent.IsValid())
		{
			AActor* Actor = SelectedComponent->PrimComponent.Get()->GetOwner();
			OutActors.AddUnique(Actor);
			if (OutLevels)
				OutLevels->AddUnique(Actor->GetLevel());
		}
	}
}

bool GetPackageNameForMergeAction(const FString& DefaultPackageName, FString& OutPackageName)
{
	if (DefaultPackageName.Len() > 0)
	{
		const FString DefaultPath = FPackageName::GetLongPackagePath(DefaultPackageName);
		const FString DefaultName = FPackageName::GetShortName(DefaultPackageName);

		// Initialize SaveAssetDialog config
		FSaveAssetDialogConfig SaveAssetDialogConfig;
		SaveAssetDialogConfig.DialogTitleOverride = LOCTEXT("CreateMergedActorTitle", "Create Merged Actor");
		SaveAssetDialogConfig.DefaultPath = DefaultPath;
		SaveAssetDialogConfig.DefaultAssetName = DefaultName;
		SaveAssetDialogConfig.ExistingAssetPolicy = ESaveAssetDialogExistingAssetPolicy::AllowButWarn;
		SaveAssetDialogConfig.AssetClassNames = { UStaticMesh::StaticClass()->GetClassPathName() };

		FContentBrowserModule& ContentBrowserModule = FModuleManager::LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		FString SaveObjectPath = ContentBrowserModule.Get().CreateModalSaveAssetDialog(SaveAssetDialogConfig);
		if (!SaveObjectPath.IsEmpty())
		{
			OutPackageName = FPackageName::ObjectPathToPackageName(SaveObjectPath);
			return true;
		}
		else
		{
			return false;
		}
	}
	else
	{
		OutPackageName = DefaultPackageName;
		return true;
	}
}

void FComponentSelectionControl::UpdateSelectedCompnentsAndListBox()
{
	StoreCheckBoxState();
	UpdateSelectedStaticMeshComponents();
	ComponentsListView->ClearSelection();
	ComponentsListView->RequestListRefresh();
}

void FComponentSelectionControl::StoreCheckBoxState() 
{
	StoredCheckBoxStates.Empty();

	// Loop over selected mesh component and store its checkbox state
	for (TSharedPtr<FMergeComponentData> SelectedComponent : SelectedComponents)
	{
		UPrimitiveComponent* PrimComponent = SelectedComponent->PrimComponent.Get();
		const ECheckBoxState State = SelectedComponent->bShouldIncorporate ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
		StoredCheckBoxStates.Add(PrimComponent, State);
	}
}

void FComponentSelectionControl::UpdateSelectedStaticMeshComponents()
{
	BuildMergeComponentDataFromSelection(SelectedComponents, bAllowShapeComponents);

	// Count number of selected objects & Update check state based on previous data
	NumSelectedMeshComponents = 0;

	for (TSharedPtr<FMergeComponentData>& ComponentData : SelectedComponents)
	{
		ECheckBoxState* StoredState = StoredCheckBoxStates.Find(ComponentData->PrimComponent.Get());
		if (StoredState != nullptr)
		{
			ComponentData->bShouldIncorporate = (*StoredState == ECheckBoxState::Checked);
		}

		// Keep count of selected meshes
		const bool bIsMesh = (Cast<UStaticMeshComponent>(ComponentData->PrimComponent.Get()) != nullptr);
		if (ComponentData->bShouldIncorporate && bIsMesh)
		{
			NumSelectedMeshComponents++;
		}
	}
}

TSharedRef<ITableRow> FComponentSelectionControl::MakeComponentListItemWidget(TSharedPtr<FMergeComponentData> ComponentData, const TSharedRef<STableViewBase>& OwnerTable)
{
	check(ComponentData->PrimComponent != nullptr);
	
	// If box should be enabled
	bool bEnabled = true;
	bool bIsMesh = false;

	// See if we stored a checkbox state for this mesh component, and set accordingly
	ECheckBoxState State = bEnabled ? ECheckBoxState::Checked : ECheckBoxState::Unchecked;
	ECheckBoxState* StoredState = StoredCheckBoxStates.Find(ComponentData->PrimComponent.Get());
	if (StoredState)
	{
		State = *StoredState;
	}

	if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(ComponentData->PrimComponent.Get()))
	{
		bEnabled = (StaticMeshComponent->GetStaticMesh() != nullptr);
		bIsMesh = true;
	}


	return SNew(STableRow<TSharedPtr<FMergeComponentData>>, OwnerTable)
		[
			SNew(SBox)
			[
				// Disable UI element if this static mesh component has invalid static mesh data
				SNew(SHorizontalBox)
				.IsEnabled(bEnabled)
				+ SHorizontalBox::Slot()
				.AutoWidth()
				[
					SNew(SCheckBox)
					.IsChecked(State)
					.ToolTipText(LOCTEXT("IncorporateCheckBoxToolTip", "When ticked the Component will be incorporated into the merge"))

					.OnCheckStateChanged_Lambda([this, ComponentData, bIsMesh](ECheckBoxState NewState)
					{
						ComponentData->bShouldIncorporate = (NewState == ECheckBoxState::Checked);

						if (bIsMesh)
						{
							this->NumSelectedMeshComponents += (NewState == ECheckBoxState::Checked) ? 1 : -1;
						}
					})
				]

				+ SHorizontalBox::Slot()
				.Padding(5.0, 0, 0, 0)
				.AutoWidth()
				[
					SNew(STextBlock)
					.Text_Lambda([ComponentData]() -> FText
					{
						if (ComponentData.IsValid() && ComponentData->PrimComponent.IsValid())
						{
							const FString OwningActorName = ComponentData->PrimComponent->GetOwner()->GetName();
							const FString ComponentName = ComponentData->PrimComponent->GetName();
							FString ComponentInfo;
							if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(ComponentData->PrimComponent.Get()))
							{
								ComponentInfo = (StaticMeshComponent->GetStaticMesh() != nullptr) ? StaticMeshComponent->GetStaticMesh()->GetName() : TEXT("No Static Mesh Available");
							}
							else if (UShapeComponent* ShapeComponent = Cast<UShapeComponent>(ComponentData->PrimComponent.Get()))
							{
								ComponentInfo = ShapeComponent->GetClass()->GetName();
							}

							return FText::FromString(OwningActorName + " - " + ComponentInfo + " - " + ComponentName);
						}

						return FText::FromString("Invalid Actor");
					})
				]
			]
		];

}

#undef LOCTEXT_NAMESPACE