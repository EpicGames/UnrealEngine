// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Components/PrimitiveComponent.h"
#include "Widgets/Views/SListView.h"
#include "Widgets/Views/STableViewBase.h"

enum class ECheckBoxState : uint8;

/** Data structure used to keep track of the selected mesh components, and whether or not they should be incorporated in the merge */
struct FMergeComponentData
{
	FMergeComponentData(UPrimitiveComponent* InPrimComponent)
		: PrimComponent(InPrimComponent)
		, bShouldIncorporate(true)
	{}

	/** Component extracted from selected actors */
	TWeakObjectPtr<UPrimitiveComponent> PrimComponent;
	/** Flag determining whether or not this component should be incorporated into the merge */
	bool bShouldIncorporate;
};

void BuildMergeComponentDataFromSelection(TArray<TSharedPtr<FMergeComponentData>>& OutComponentsData, bool bAllowShapeComponents);
void BuildActorsListFromMergeComponentsData(const TArray<TSharedPtr<FMergeComponentData>>& InComponentsData, TArray<AActor*>& OutActors, TArray<ULevel*>* OutLevels = nullptr);
bool GetPackageNameForMergeAction(const FString& DefaultPackageName, FString& OutPackageName);

struct FComponentSelectionControl
{

	/** Delegate for the creation of the list view item's widget */
	TSharedRef<ITableRow> MakeComponentListItemWidget(TSharedPtr<FMergeComponentData> ComponentData, const TSharedRef<STableViewBase>& OwnerTable);

	
	void UpdateSelectedCompnentsAndListBox();

	/** Stores the individual check box states for the currently selected mesh components */
	void StoreCheckBoxState();

	/** Updates SelectedMeshComponent array according to retrieved mesh components from editor selection*/
	void  UpdateSelectedStaticMeshComponents();

	/** List of mesh components extracted from editor selection */
	TArray<TSharedPtr<FMergeComponentData>> SelectedComponents;
	/** List view ui element */
	TSharedPtr<SListView<TSharedPtr<FMergeComponentData>>> ComponentsListView;
	/** Map of keeping track of checkbox states for each selected component (used to restore state when listview is refreshed) */
	TMap<UPrimitiveComponent*, ECheckBoxState> StoredCheckBoxStates;
	
	/** Number of selected static mesh components */
	int32 NumSelectedMeshComponents = 0;

	/** Allow selection of ShapeComponents **/
	bool bAllowShapeComponents = true;
};
