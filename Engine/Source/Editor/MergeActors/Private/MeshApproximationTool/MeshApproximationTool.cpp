// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshApproximationTool/MeshApproximationTool.h"
#include "Misc/Paths.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "GameFramework/Actor.h"
#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Materials/Material.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "Misc/MessageDialog.h"
#include "MeshApproximationTool/SMeshApproximationDialog.h"
#include "MeshUtilities.h"
#include "MaterialUtilities.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IMeshReductionInterfaces.h"
#include "IMeshMergeUtilities.h"
#include "MeshMergeModule.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "AssetToolsModule.h"
#include "ObjectTools.h"
#include "Algo/ForEach.h"
#include "Algo/Transform.h"

#include "IGeometryProcessingInterfacesModule.h"
#include "GeometryProcessingInterfaces/ApproximateActors.h"

#include "SMeshApproximationDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshApproximationTool)

#define LOCTEXT_NAMESPACE "MeshApproximationTool"

bool UMeshApproximationSettingsObject::bInitialized = false;
UMeshApproximationSettingsObject* UMeshApproximationSettingsObject::DefaultSettings = nullptr;



FMeshApproximationTool::FMeshApproximationTool()
{
	SettingsObject = UMeshApproximationSettingsObject::Get();
}

FMeshApproximationTool::~FMeshApproximationTool()
{
	UMeshApproximationSettingsObject::Destroy();
	SettingsObject = nullptr;
}

TSharedRef<SWidget> FMeshApproximationTool::GetWidget()
{
	SAssignNew(ProxyDialog, SMeshApproximationDialog, this);
	return ProxyDialog.ToSharedRef();
}

FName FMeshApproximationTool::GetIconName() const
{
	return "MergeActors.Approximate";
}

FText FMeshApproximationTool::GetToolNameText() const
{
	return LOCTEXT("MeshApproximationToolName", "Approximate");
}

FText FMeshApproximationTool::GetTooltipText() const
{
	return LOCTEXT("MeshApproximationToolTooltip", "Merge source actors meshes and perform an approximation pass. Will generate a single static mesh with baked textures.");
}

FString FMeshApproximationTool::GetDefaultPackageName() const
{
	FString PackageName;

	USelection* SelectedActors = GEditor->GetSelectedActors();

	// Iterate through selected actors and find first static mesh asset
	// Use this static mesh path as destination package name for a merged mesh
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			TInlineComponentArray<UStaticMeshComponent*> SMComponets;
			Actor->GetComponents(SMComponets);
			for (UStaticMeshComponent* Component : SMComponets)
			{
				if (Component->GetStaticMesh())
				{
					PackageName = FPackageName::GetLongPackagePath(Component->GetStaticMesh()->GetOutermost()->GetName());
					PackageName += FString(TEXT("/APPROX_")) + Component->GetStaticMesh()->GetName();
					break;
				}
			}
		}

		if (!PackageName.IsEmpty())
		{
			break;
		}
	}

	if (PackageName.IsEmpty())
	{
		PackageName = FPackageName::FilenameToLongPackageName(FPaths::ProjectContentDir() + TEXT("APPROX"));
		PackageName = MakeUniqueObjectName(NULL, UPackage::StaticClass(), *PackageName).ToString();
	}

	return PackageName;
}


const TArray<TSharedPtr<FMergeComponentData>>& FMeshApproximationTool::GetSelectedComponentsInWidget() const
{
	return ProxyDialog->GetSelectedComponents();
}


static void ReplaceSourceActorsByApproximationMeshes(TArray<UObject*>& NewAssetsToSync, ULevel* Level, TArray<AActor*>& Actors)
{
	UStaticMesh* MergedMesh = nullptr;
	if (NewAssetsToSync.FindItemByClass(&MergedMesh))
	{
		Level->Modify();

		UWorld* World = Level->OwningWorld;
		FActorSpawnParameters Params;
		Params.OverrideLevel = Level;
		FRotator MergedActorRotation(ForceInit);
		// The pivot of the merged mesh is always at the origin
		FVector MergedActorLocation(0, 0, 0);

		AStaticMeshActor* MergedActor = World->SpawnActor<AStaticMeshActor>(MergedActorLocation, MergedActorRotation, Params);
		MergedActor->GetStaticMeshComponent()->SetStaticMesh(MergedMesh);
		MergedActor->SetActorLabel(MergedMesh->GetName());
		World->UpdateCullDistanceVolumes(MergedActor, MergedActor->GetStaticMeshComponent());
		GEditor->SelectNone(true, true);
		GEditor->SelectActor(MergedActor, true, true);

		// Remove source actors
		for (AActor* Actor : Actors)
		{
			Actor->Destroy();
		}
	}
}

bool FMeshApproximationTool::RunMerge(const FString& PackageName, const TArray<TSharedPtr<FMergeComponentData>>& SelectedComponents)
{
	TArray<AActor*> Actors;
	TArray<ULevel*> UniqueLevels;
	TArray<UObject*> AssetsToSync;

	BuildActorsListFromMergeComponentsData(SelectedComponents, Actors, bReplaceSourceActors ? &UniqueLevels : nullptr);

	// This restriction is only for replacement of selected actors with merged mesh actor
	if (UniqueLevels.Num() > 1 && bReplaceSourceActors)
	{
		FText Message = NSLOCTEXT("UnrealEd", "FailedToMergeActorsSublevels_Msg", "The selected actors should be in the same level");
		const FText Title = NSLOCTEXT("UnrealEd", "FailedToMergeActors_Title", "Unable to merge actors");
		FMessageDialog::Open(EAppMsgType::Ok, Message, Title);
		return false;
	}

	const FMeshApproximationSettings& UseSettings = SettingsObject->Settings;

	IGeometryProcessingInterfacesModule& GeomProcInterfaces = FModuleManager::Get().LoadModuleChecked<IGeometryProcessingInterfacesModule>("GeometryProcessingInterfaces");
	IGeometryProcessing_ApproximateActors* ApproxActorsAPI = GeomProcInterfaces.GetApproximateActorsImplementation();

	//
	// Construct options for ApproximateActors operation
	//

	IGeometryProcessing_ApproximateActors::FOptions Options = ApproxActorsAPI->ConstructOptions(UseSettings);

	TArray<UPrimitiveComponent*> PrimitiveComponents;
	auto IsValidPrimitiveComponent = [](const TSharedPtr<FMergeComponentData>& Component) { return Component.IsValid() && Component.Get()->bShouldIncorporate; };
	auto GetPrimitiveComponent = [](const TSharedPtr<FMergeComponentData>& Component) { return Component.Get()->PrimComponent.Get(); };
	Algo::TransformIf(SelectedComponents, PrimitiveComponents, IsValidPrimitiveComponent, GetPrimitiveComponent);

	// Compute texel density if needed, depending on the TextureSizingType setting
	if (UseSettings.MaterialSettings.ResolveTexelDensity(PrimitiveComponents, Options.MeshTexelDensity))
	{
		Options.TextureSizePolicy = IGeometryProcessing_ApproximateActors::ETextureSizePolicy::TexelDensity;
	}

	// Use temp packages - Needed to allow proper replacement of existing assets (performed below)
	const FString NewAssetNamePrefix(TEXT("NEWASSET_"));
	FString PackagePath = FPackageName::GetLongPackagePath(PackageName);
	FString AssetName = FPackageName::GetLongPackageAssetName(PackageName);
	Options.BasePackagePath = PackagePath / NewAssetNamePrefix + AssetName;

	// Extracting static mesh components from the selected mesh components in the dialog
	IGeometryProcessing_ApproximateActors::FInput Input;

	for (const TSharedPtr<FMergeComponentData>& SelectedComponent : SelectedComponents)
	{
		// Determine whether or not this component should be incorporated according the user settings
		if (SelectedComponent->bShouldIncorporate && SelectedComponent->PrimComponent.IsValid())
		{
			if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(SelectedComponent->PrimComponent.Get()))
			{
				if (StaticMeshComponent->GetStaticMesh() != nullptr)
				{
					Input.Components.Add(StaticMeshComponent);
				}
			}
		}
	}
	
	// Run actor approximation computation
	IGeometryProcessing_ApproximateActors::FResults Results;
	ApproxActorsAPI->ApproximateActors(Input, Options, Results);

	auto ProcessNewAsset = [&PackagePath, &NewAssetNamePrefix](UObject* NewAsset)
	{
		// Move asset out of the temp package and into its final package
		{
			FString AssetName = NewAsset->GetName();
			AssetName.RemoveFromStart(NewAssetNamePrefix);
			FString TargetPackageName = FPaths::Combine(PackagePath, AssetName);

			UPackage* TargetPackage = CreatePackage(*TargetPackageName);
			TargetPackage->FullyLoad();

			// Remplace existing asset
			UObject* AssetToReplace = StaticFindObjectFast(UObject::StaticClass(), TargetPackage, *AssetName);
			if (AssetToReplace)
			{
				// Replace references
				TArray<UObject*> ObjectsToReplace(&AssetToReplace, 1);
				ObjectTools::ForceReplaceReferences(NewAsset, ObjectsToReplace);

				// Move the previous asset to the transient package
				AssetToReplace->Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors | REN_NonTransactional);
			}

			UPackage* TempPackage = NewAsset->GetPackage();

			// Rename the asset to its final destination
			NewAsset->Rename(*AssetName, TargetPackage, REN_DontCreateRedirectors | REN_NonTransactional | REN_AllowPackageLinkerMismatch);
			check(NewAsset->HasAllFlags(RF_Public | RF_Standalone));

			// Clean up flags on the temp package. It is not useful anymore.
			TempPackage->ClearDirtyFlag();
			TempPackage->SetFlags(RF_Transient);
			TempPackage->ClearFlags(RF_Public | RF_Standalone);
		}

		// Notify Asset Browser about new Assets
		FAssetRegistryModule::AssetCreated(NewAsset);
	};

	if (Results.ResultCode != IGeometryProcessing_ApproximateActors::EResultCode::Success)
	{
		FText Message;
		FText Title = LOCTEXT("ApproximateActorsFailed_Title", "Failed to merge actors");

		switch (Results.ResultCode)
		{
		case IGeometryProcessing_ApproximateActors::EResultCode::MeshGenerationFailed:
			Message = LOCTEXT("ApproximateActors_MeshGeneratedFailed", "Mesh generation failed. Please review the merge settings and look for additional errors in the console log.");
			break;
		case IGeometryProcessing_ApproximateActors::EResultCode::MaterialGenerationFailed:
			Message = LOCTEXT("ApproximateActors_MaterialGenerationFailed", "Material generation failed. Please review the merge settings and look for additional errors in the console log.");
			break;
		case IGeometryProcessing_ApproximateActors::EResultCode::UnknownError:
			Message = LOCTEXT("ApproximateActors_UnknownError", "Unknown merge error. Please review the merge settings and look for additional errors in the console log.");
			break;
		}		
		
		FMessageDialog::Open(EAppMsgType::Ok, Message, Title);
		return false;
	}

	Algo::ForEach(Results.NewMeshAssets, ProcessNewAsset);
	Algo::ForEach(Results.NewMaterials, ProcessNewAsset);
	Algo::ForEach(Results.NewTextures, ProcessNewAsset);

	if (ensure(Results.NewMeshAssets.Num() == 1))
	{
		TArray<UObject*> NewAssetsToSync;
		NewAssetsToSync.Add(Results.NewMeshAssets[0]);
		if (bReplaceSourceActors && UniqueLevels[0])
		{
			ReplaceSourceActorsByApproximationMeshes(NewAssetsToSync, UniqueLevels[0], Actors);
		}
	}
	
	return true;
}



#undef LOCTEXT_NAMESPACE

