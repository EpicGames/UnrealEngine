// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "MeshMerge/MeshApproximationSettings.h"
#include "MergeActorsTool.h"

#include "MeshApproximationTool.generated.h"

class SMeshApproximationDialog;

/** Singleton wrapper to allow for using the setting structure in SSettingsView */
UCLASS(config = Engine)
class UMeshApproximationSettingsObject : public UObject
{
	GENERATED_BODY()
public:
	UMeshApproximationSettingsObject()
	{
	}

	static UMeshApproximationSettingsObject* Get()
	{	
		// This is a singleton, duplicate default object
		if (!bInitialized)
		{
			DefaultSettings = DuplicateObject(GetMutableDefault<UMeshApproximationSettingsObject>(), nullptr);
			DefaultSettings->AddToRoot();
			bInitialized = true;
		}

		return DefaultSettings;
	}

	static void Destroy()
	{
		if (bInitialized)
		{
			if (UObjectInitialized() && DefaultSettings)
			{
				DefaultSettings->RemoveFromRoot();
				DefaultSettings->MarkAsGarbage();
			}

			DefaultSettings = nullptr;
			bInitialized = false;
		}
	}

protected:
	static bool bInitialized;
	static UMeshApproximationSettingsObject* DefaultSettings;
public:
	UPROPERTY(EditAnywhere, meta = (ShowOnlyInnerProperties), Category = ApproximationSettings)
	FMeshApproximationSettings Settings;
};

/**
* Mesh Proxy Tool
*/
class FMeshApproximationTool : public FMergeActorsTool
{
	friend class SMeshApproximationDialog;

public:
	FMeshApproximationTool();
	~FMeshApproximationTool();

	// IMergeActorsTool interface
	virtual TSharedRef<SWidget> GetWidget() override;
	virtual FName GetIconName() const override;
	virtual FText GetToolNameText() const override;
	virtual FText GetTooltipText() const override;
	virtual FString GetDefaultPackageName() const override;

protected:
	virtual bool RunMerge(const FString& PackageName, const TArray<TSharedPtr<FMergeComponentData>>& SelectedComponents) override;
	virtual const TArray<TSharedPtr<FMergeComponentData>>& GetSelectedComponentsInWidget() const override;

protected:

	/** Pointer to the mesh merging dialog containing settings for the merge */
	TSharedPtr<SMeshApproximationDialog> ProxyDialog;

	/** Pointer to singleton settings object */
	UMeshApproximationSettingsObject* SettingsObject;
};

