// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshApproximationTool/SMeshApproximationDialog.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "Engine/Selection.h"
#include "MeshApproximationTool/MeshApproximationTool.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styling/SlateTypes.h"
#include "SlateOptMacros.h"
#include "UObject/UnrealType.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SMeshApproximationDialog"

//////////////////////////////////////////////////////////////////////////
// SMeshApproximationDialog
SMeshApproximationDialog::SMeshApproximationDialog()
{
    MergeStaticMeshComponentsLabel = LOCTEXT("CreateProxyMeshComponentsLabel", "Mesh components used to compute the proxy mesh:");
	SelectedComponentsListBoxToolTip = LOCTEXT("CreateProxyMeshSelectedComponentsListBoxToolTip", "The selected mesh components will be used to compute the proxy mesh");
    DeleteUndoLabel = LOCTEXT("DeleteUndo", "Insufficient mesh components found for Mesh Approximation.");
}

SMeshApproximationDialog::~SMeshApproximationDialog()
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void  SMeshApproximationDialog::Construct(const FArguments& InArgs, FMeshApproximationTool* InTool)
{
	checkf(InTool != nullptr, TEXT("Invalid owner tool supplied"));
	Tool = InTool;

	SMeshProxyCommonDialog::Construct(SMeshProxyCommonDialog::FArguments());

	ProxySettings = UMeshApproximationSettingsObject::Get();
	SettingsView->SetObject(ProxySettings);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION



#undef LOCTEXT_NAMESPACE
