// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"

#include "MergeProxyUtils/Utils.h"
#include "MergeProxyUtils/SMeshProxyCommonDialog.h"

class FMeshApproximationTool;
class UMeshApproximationSettingsObject;
class UObject;

/*-----------------------------------------------------------------------------
SMeshApproximationDialog  
-----------------------------------------------------------------------------*/

class SMeshApproximationDialog : public SMeshProxyCommonDialog
{
public:
	SLATE_BEGIN_ARGS(SMeshApproximationDialog)
	{
	}
	SLATE_END_ARGS()

public:
	/** **/
	SMeshApproximationDialog();
	~SMeshApproximationDialog();

	/** SWidget functions */
	void Construct(const FArguments& InArgs, FMeshApproximationTool* InTool);

private:
	/** Owning mesh merging tool */
	FMeshApproximationTool* Tool;

	/** Cached pointer to mesh merging setting singleton object */
	UMeshApproximationSettingsObject* ProxySettings;
};


