// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshInstancingTool/MeshInstancingTool.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "Misc/MessageDialog.h"
#include "MeshUtilities.h"
#include "MeshInstancingTool/SMeshInstancingDialog.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ScopedTransaction.h"
#include "MeshMergeModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshInstancingTool)

#define LOCTEXT_NAMESPACE "MeshInstancingTool"

bool UMeshInstancingSettingsObject::bInitialized = false;
UMeshInstancingSettingsObject* UMeshInstancingSettingsObject::DefaultSettings = nullptr;

FMeshInstancingTool::FMeshInstancingTool()
{
	bAllowShapeComponents = false;
	SettingsObject = UMeshInstancingSettingsObject::Get();
}

FMeshInstancingTool::~FMeshInstancingTool()
{
	UMeshInstancingSettingsObject::Destroy();
	SettingsObject = nullptr;
}

TSharedRef<SWidget> FMeshInstancingTool::GetWidget()
{
	SAssignNew(InstancingDialog, SMeshInstancingDialog, this);
	return InstancingDialog.ToSharedRef();
}

FName FMeshInstancingTool::GetIconName() const
{
	return "MergeActors.MeshInstancingTool";
}

FText FMeshInstancingTool::GetToolNameText() const
{
	return LOCTEXT("MeshInstancingToolName", "Batch");
}

FText FMeshInstancingTool::GetTooltipText() const
{
	return LOCTEXT("MeshInstancingToolTooltip", "Batch the source actors components to use instancing as much as possible. Will generate an actor with instanced static mesh component(s).");
}

FString FMeshInstancingTool::GetDefaultPackageName() const
{
	return FString();
}

const TArray<TSharedPtr<FMergeComponentData>>& FMeshInstancingTool::GetSelectedComponentsInWidget() const
{
	return InstancingDialog->GetSelectedComponents();
}

bool FMeshInstancingTool::RunMerge(const FString& PackageName, const TArray<TSharedPtr<FMergeComponentData>>& SelectedComponents)
{
	const IMeshMergeUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
	TArray<AActor*> Actors;
	TArray<ULevel*> UniqueLevels;

	BuildActorsListFromMergeComponentsData(SelectedComponents, Actors, &UniqueLevels);

	// This restriction is only for replacement of selected actors with merged mesh actor
	if (UniqueLevels.Num() > 1)
	{
		FText Message = NSLOCTEXT("UnrealEd", "FailedToInstanceActorsSublevels_Msg", "The selected actors should be in the same level");
		FText Title = NSLOCTEXT("UnrealEd", "FailedToInstanceActors_Title", "Unable to replace actors with instanced meshes");
		FMessageDialog::Open(EAppMsgType::Ok, Message, Title);
		return false;
	}

	// Instance...
	{
		FScopedSlowTask SlowTask(0, LOCTEXT("MergingActorsSlowTask", "Instancing actors..."));
		SlowTask.MakeDialog();

		// Extracting static mesh components from the selected mesh components in the dialog
		TArray<UPrimitiveComponent*> ComponentsToMerge;

		for ( const TSharedPtr<FMergeComponentData>& SelectedComponent : SelectedComponents)
		{
			// Determine whether or not this component should be incorporated according the user settings
			if (SelectedComponent->bShouldIncorporate && SelectedComponent->PrimComponent.IsValid())
			{
				ComponentsToMerge.Add(SelectedComponent->PrimComponent.Get());
			}
		}

		if (ComponentsToMerge.Num())
		{
			// spawn the actor that will contain out instances
			UWorld* World = ComponentsToMerge[0]->GetWorld();
			checkf(World != nullptr, TEXT("Invalid World retrieved from Mesh components"));
			const bool bActuallyMerge = true;
			MeshUtilities.MergeComponentsToInstances(ComponentsToMerge, World, UniqueLevels[0], SettingsObject->Settings, bActuallyMerge, bReplaceSourceActors, nullptr);
		}
	}

	if (InstancingDialog)
	{
		InstancingDialog->Reset();
	}

	return true;
}

FText FMeshInstancingTool::GetPredictedResultsText()
{
	const IMeshMergeUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
	const TArray<TSharedPtr<FMergeComponentData>>& SelectedComponents = InstancingDialog->GetSelectedComponents();
	TArray<AActor*> Actors;
	TArray<ULevel*> UniqueLevels;

	BuildActorsListFromMergeComponentsData(SelectedComponents, Actors, &UniqueLevels);

	// This restriction is only for replacement of selected actors with merged mesh actor
	if (UniqueLevels.Num() > 1)
	{
		return NSLOCTEXT("UnrealEd", "FailedToInstanceActorsSublevels_Msg", "The selected actors should be in the same level");
	}

	// Extracting static mesh components from the selected mesh components in the dialog
	TArray<UPrimitiveComponent*> ComponentsToMerge;

	for ( const TSharedPtr<FMergeComponentData>& SelectedComponent : SelectedComponents)
	{
		// Determine whether or not this component should be incorporated according the user settings
		if (SelectedComponent->bShouldIncorporate)
		{
			ComponentsToMerge.Add(SelectedComponent->PrimComponent.Get());
		}
	}
		
	FText OutResultsText;
	if(ComponentsToMerge.Num() > 0 && ComponentsToMerge[0])
	{
		UWorld* World = ComponentsToMerge[0]->GetWorld();
		checkf(World != nullptr, TEXT("Invalid World retrieved from Mesh components"));
		const bool bActuallyMerge = false;
		MeshUtilities.MergeComponentsToInstances(ComponentsToMerge, World, UniqueLevels.Num() > 0 ? UniqueLevels[0] : nullptr, SettingsObject->Settings, bActuallyMerge, bReplaceSourceActors, &OutResultsText);
	}
	else
	{
		OutResultsText = LOCTEXT("InstanceMergePredictedResultsNone", "The current settings will not result in any instanced meshes being created");
	}

	return OutResultsText;
}
#undef LOCTEXT_NAMESPACE
