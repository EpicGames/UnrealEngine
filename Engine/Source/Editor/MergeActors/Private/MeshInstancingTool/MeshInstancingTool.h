// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Widgets/SWidget.h"
#include "MeshMerge/MeshInstancingSettings.h"
#include "MergeActorsTool.h"

#include "MeshInstancingTool.generated.h"

class SMeshInstancingDialog;

/** Singleton wrapper to allow for using the setting structure in SSettingsView */
UCLASS(config = Engine)
class UMeshInstancingSettingsObject : public UObject
{
	GENERATED_BODY()
public:
	UMeshInstancingSettingsObject()
	{
	}

	static UMeshInstancingSettingsObject* Get()
	{
		if (!bInitialized)
		{
			DefaultSettings = DuplicateObject(GetMutableDefault<UMeshInstancingSettingsObject>(), nullptr);
			DefaultSettings->AddToRoot();
			bInitialized = true;
		}

		return DefaultSettings;
	}

	static void Destroy()
	{
		if (bInitialized)
		{
			if (UObjectInitialized() && DefaultSettings)
			{
				DefaultSettings->RemoveFromRoot();
				DefaultSettings->MarkAsGarbage();
			}

			DefaultSettings = nullptr;
			bInitialized = false;
		}
	}

protected:
	static bool bInitialized;
	// This is a singleton, duplicate default object
	static UMeshInstancingSettingsObject* DefaultSettings;

public:
	UPROPERTY(EditAnywhere, meta = (ShowOnlyInnerProperties), Category = MergeSettings)
	FMeshInstancingSettings Settings;
};

/**
 * Mesh Instancing Tool
 */
class FMeshInstancingTool : public FMergeActorsTool
{
	friend class SMeshInstancingDialog;

public:

	FMeshInstancingTool();
	~FMeshInstancingTool();

	// IMergeActorsTool interface
	virtual TSharedRef<SWidget> GetWidget() override;
	virtual FName GetIconName() const override;
	virtual FText GetToolNameText() const override;
	virtual FText GetTooltipText() const override;
	virtual FString GetDefaultPackageName() const override;

	/** Runs the merging logic to determine predicted results */
	FText GetPredictedResultsText();

protected:
	virtual bool RunMerge(const FString& PackageName, const TArray<TSharedPtr<FMergeComponentData>>& SelectedComponents) override;
	virtual const TArray<TSharedPtr<FMergeComponentData>>& GetSelectedComponentsInWidget() const override;

private:
	/** Pointer to the mesh instancing dialog containing settings for the merge */
	TSharedPtr<SMeshInstancingDialog> InstancingDialog;

	/** Pointer to singleton settings object */
	UMeshInstancingSettingsObject* SettingsObject;
};
