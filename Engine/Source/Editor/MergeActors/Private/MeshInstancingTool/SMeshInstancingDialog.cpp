// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshInstancingTool/SMeshInstancingDialog.h"
#include "MeshInstancingTool/MeshInstancingTool.h"
#include "SlateOptMacros.h"

#include "IDetailsView.h"

//////////////////////////////////////////////////////////////////////////
// SMeshInstancingDialog
SMeshInstancingDialog::SMeshInstancingDialog()
{
	ComponentSelectionControl.bAllowShapeComponents = false;
}

SMeshInstancingDialog::~SMeshInstancingDialog()
{

}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMeshInstancingDialog::Construct(const FArguments& InArgs, FMeshInstancingTool* InTool)
{
	checkf(InTool != nullptr, TEXT("Invalid owner tool supplied"));
	Tool = InTool;

	SMeshProxyCommonDialog::Construct(SMeshProxyCommonDialog::FArguments());

	InstancingSettings = UMeshInstancingSettingsObject::Get();
	SettingsView->SetObject(InstancingSettings);

	Reset();
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION

FText SMeshInstancingDialog::GetPredictedResultsTextInternal() const
{
	return Tool->GetPredictedResultsText();
}
