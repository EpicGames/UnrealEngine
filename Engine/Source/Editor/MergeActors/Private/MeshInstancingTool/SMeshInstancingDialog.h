// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Components/StaticMeshComponent.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

#include "MergeProxyUtils/SMeshProxyCommonDialog.h"

class FMeshInstancingTool;
class UMeshInstancingSettingsObject;

/*-----------------------------------------------------------------------------
   SMeshInstancingDialog
-----------------------------------------------------------------------------*/
class SMeshInstancingDialog : public SMeshProxyCommonDialog
{
public:
	SLATE_BEGIN_ARGS(SMeshInstancingDialog)
	{
	}

	SLATE_END_ARGS()

public:
	/** **/
	SMeshInstancingDialog();
	~SMeshInstancingDialog();

	/** SWidget functions */
	void Construct(const FArguments& InArgs, FMeshInstancingTool* InTool);

protected:
	/** Refresh the predicted results text */;
	FText GetPredictedResultsTextInternal() const override;

private:
	/** Owning mesh instancing tool */
	FMeshInstancingTool* Tool;

	/** Cached pointer to mesh instancing setting singleton object */
	UMeshInstancingSettingsObject* InstancingSettings;
};
