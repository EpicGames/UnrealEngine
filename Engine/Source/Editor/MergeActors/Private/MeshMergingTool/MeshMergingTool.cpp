// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshMergingTool/MeshMergingTool.h"
#include "Misc/Paths.h"
#include "Misc/ScopedSlowTask.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "Misc/MessageDialog.h"
#include "MeshUtilities.h"
#include "MeshMergingTool/SMeshMergingDialog.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "ScopedTransaction.h"
#include "MeshMergeModule.h"
#include "ComponentReregisterContext.h"
#include "ObjectTools.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshMergingTool)

#define LOCTEXT_NAMESPACE "MeshMergingTool"

UMeshMergingSettingsObject* UMeshMergingSettingsObject::DefaultSettings = nullptr;
bool UMeshMergingSettingsObject::bInitialized = false;

FMeshMergingTool::FMeshMergingTool()
{
	SettingsObject = UMeshMergingSettingsObject::Get();
}

FMeshMergingTool::~FMeshMergingTool()
{
	UMeshMergingSettingsObject::Destroy();
	SettingsObject = nullptr;
}

TSharedRef<SWidget> FMeshMergingTool::GetWidget()
{
	SAssignNew(MergingDialog, SMeshMergingDialog, this);
	return MergingDialog.ToSharedRef();
}

FName FMeshMergingTool::GetIconName() const
{
	return "MergeActors.MeshMergingTool";
}

FText FMeshMergingTool::GetToolNameText() const
{
	return LOCTEXT("MeshMergingToolName", "Merge");
}

FText FMeshMergingTool::GetTooltipText() const
{
	return LOCTEXT("MeshMergingToolTooltip", "Merge the source actors components to generate a single mesh. No simplification pass is performed. Will generate a single static mesh & optionally bake down textures.");
}

FString FMeshMergingTool::GetDefaultPackageName() const
{
	FString PackageName = FPackageName::FilenameToLongPackageName(FPaths::ProjectContentDir() + TEXT("MERGED"));

	USelection* SelectedActors = GEditor->GetSelectedActors();
	// Iterate through selected actors and find first static mesh asset
	// Use this static mesh path as destination package name for a merged mesh
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			FString ActorName = Actor->GetName();
			PackageName = FString::Printf(TEXT("%s_%s"), *PackageName, *ActorName);
			break;
		}
	}

	if (PackageName.IsEmpty())
	{
		PackageName = MakeUniqueObjectName(NULL, UPackage::StaticClass(), *PackageName).ToString();
	}

	return PackageName;
}

const TArray<TSharedPtr<FMergeComponentData>>& FMeshMergingTool::GetSelectedComponentsInWidget() const
{
	return MergingDialog->GetSelectedComponents();
}

bool FMeshMergingTool::RunMerge(const FString& PackageName, const TArray<TSharedPtr<FMergeComponentData>>& SelectedComponents)
{
	const IMeshMergeUtilities& MeshUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();
	TArray<AActor*> Actors;
	TArray<ULevel*> UniqueLevels;

	BuildActorsListFromMergeComponentsData(SelectedComponents, Actors, bReplaceSourceActors ? &UniqueLevels : nullptr);

	// This restriction is only for replacement of selected actors with merged mesh actor
	if (UniqueLevels.Num() > 1 && bReplaceSourceActors)
	{
		FText Message = NSLOCTEXT("UnrealEd", "FailedToMergeActorsSublevels_Msg", "The selected actors should be in the same level");
		const FText Title = NSLOCTEXT("UnrealEd", "FailedToMergeActors_Title", "Unable to merge actors");
		FMessageDialog::Open(EAppMsgType::Ok, Message, Title);
		return false;
	}

	FVector MergedActorLocation;
	TArray<UObject*> MergedMeshAssets;

	// The call to ObjectTools::ConsolidateObjects() can actually trigger a GC, so we keep the newly generated assets
	// in the root set using this array.
	TArray<TStrongObjectPtr<UObject>> RootedObjects;

	// Merge...
	{
		FScopedSlowTask SlowTask(0, LOCTEXT("MergingActorsSlowTask", "Merging actors..."));
		SlowTask.MakeDialog();

		// Extracting static mesh components from the selected mesh components in the dialog
		TArray<UPrimitiveComponent*> ComponentsToMerge;

		for ( const TSharedPtr<FMergeComponentData>& SelectedComponent : SelectedComponents)
		{
			// Determine whether or not this component should be incorporated according the user settings
			if (SelectedComponent->bShouldIncorporate && SelectedComponent->PrimComponent.IsValid())
			{
				ComponentsToMerge.Add(SelectedComponent->PrimComponent.Get());
			}
		}

		if (ComponentsToMerge.Num())
		{
			UWorld* World = ComponentsToMerge[0]->GetWorld();
			checkf(World != nullptr, TEXT("Invalid World retrieved from Mesh components"));
			const float ScreenAreaSize = TNumericLimits<float>::Max();

			// If the merge destination package already exists, it is possible that the mesh is already used in a scene somewhere, or its materials or even just its textures.
			// Static primitives uniform buffers could become invalid after the operation completes and lead to memory corruption. To avoid it, we force a global reregister.
			if (FindObject<UObject>(nullptr, *PackageName))
			{
				FGlobalComponentReregisterContext GlobalReregister;
				MeshUtilities.MergeComponentsToStaticMesh(ComponentsToMerge, World, SettingsObject->Settings, nullptr, nullptr, PackageName, MergedMeshAssets, MergedActorLocation, ScreenAreaSize, true);
			}
			else
			{
				MeshUtilities.MergeComponentsToStaticMesh(ComponentsToMerge, World, SettingsObject->Settings, nullptr, nullptr, PackageName, MergedMeshAssets, MergedActorLocation, ScreenAreaSize, true);
			}
		}
	}

	Algo::Transform(MergedMeshAssets, RootedObjects, [](UObject* AssetToSync) { return TStrongObjectPtr(AssetToSync); });

	if (MergedMeshAssets.Num())
	{
		FAssetRegistryModule& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");

		// MergeComponentsToStaticMesh() will have outered all assets (material instances, textures) to the static mesh package.
		// Move each of them to their own package so that they show up in the Content Browser.
		struct FAssetMoveInfo
		{
			UObject* OldAsset;
			UObject* NewAsset;
			FString AssetName;
			UPackage* AssetPackage;
		};
		TArray<FAssetMoveInfo> AssetMoveInfos;

		// Create target packages
		for (UObject* NewAsset : MergedMeshAssets)
		{
			if (NewAsset && !NewAsset->IsA<UStaticMesh>())
			{
				FString NewAssetName = NewAsset->GetName();
				FString NewAssetPackagePath = FPackageName::GetLongPackagePath(NewAsset->GetPathName());
				FString NewAssetPackageName = NewAssetPackagePath / NewAssetName;

				UPackage* NewAssetPackage = CreatePackage(*NewAssetPackageName);
				check(NewAssetPackage);
				NewAssetPackage->FullyLoad();
				NewAssetPackage->Modify();
				RootedObjects.Emplace(NewAssetPackage);

				UObject* OldAsset = FindObject<UObject>(NewAssetPackage, *NewAssetName);
				if (OldAsset)
				{
					// Move old assets out of the way
					OldAsset->Rename(nullptr, GetTransientPackage(), REN_DoNotDirty | REN_DontCreateRedirectors);
				}
				AssetMoveInfos.Add({ OldAsset, NewAsset, MoveTemp(NewAssetName), NewAssetPackage });
			}
		}

		// Replace all old asset references with the new ones
		{
			TArray<ObjectTools::FReplaceRequest> ConsolidateRequests;
			for (FAssetMoveInfo& MoveInfo : AssetMoveInfos)
			{
				if (MoveInfo.OldAsset)
				{
					ConsolidateRequests.Add({MoveInfo.NewAsset, MakeArrayView(&MoveInfo.OldAsset, 1)});
				}
			}

			if (ConsolidateRequests.Num())
			{
				TSet<UObject*> ObjectsToConsolidateWithin;
				TSet<UObject*> ObjectsToNotConsolidateWithin;
				ObjectTools::ConsolidateObjects(ConsolidateRequests, ObjectsToConsolidateWithin, ObjectsToNotConsolidateWithin, /*bShouldDeleteAfterConsolidate=*/false);
			}
		}

		// Move new assets into their target packages
		for (FAssetMoveInfo& MoveInfo : AssetMoveInfos)
		{
			MoveInfo.NewAsset->Rename(*MoveInfo.AssetName, MoveInfo.AssetPackage, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
			MoveInfo.NewAsset->SetFlags(RF_Public | RF_Standalone);
		}

		for (UObject* AssetToSync : MergedMeshAssets)
		{
			AssetRegistry.AssetCreated(AssetToSync);
			GEditor->BroadcastObjectReimported(AssetToSync);
		}

		// Also notify the content browser that the new assets exists
		FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
		ContentBrowserModule.Get().SyncBrowserToAssets(MergedMeshAssets, true);

		// Place new mesh in the world
		if (bReplaceSourceActors)
		{
			UStaticMesh* MergedMesh = nullptr;
			if (MergedMeshAssets.FindItemByClass(&MergedMesh))
			{
				const FScopedTransaction Transaction(LOCTEXT("PlaceMergedActor", "Place Merged Actor"));
				UniqueLevels[0]->Modify();

				UWorld* World = UniqueLevels[0]->OwningWorld;
				FActorSpawnParameters Params;
				Params.OverrideLevel = UniqueLevels[0];
				FRotator MergedActorRotation(ForceInit);
								
				AStaticMeshActor* MergedActor = World->SpawnActor<AStaticMeshActor>(MergedActorLocation, MergedActorRotation, Params);
				MergedActor->GetStaticMeshComponent()->SetStaticMesh(MergedMesh);
				MergedActor->SetActorLabel(MergedMesh->GetName());
				World->UpdateCullDistanceVolumes(MergedActor, MergedActor->GetStaticMeshComponent());
				GEditor->SelectNone(true, true);
				GEditor->SelectActor(MergedActor, true, true);
				// Remove source actors
				for (AActor* Actor : Actors)
				{
					Actor->Destroy();
				}
			}
		}
	}

	if (MergingDialog)
	{
		MergingDialog->Reset();
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

