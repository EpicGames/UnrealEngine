// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Object.h"
#include "Widgets/SWidget.h"
#include "MeshMerge/MeshMergingSettings.h"
#include "MergeActorsTool.h"

#include "MeshMergingTool.generated.h"

class SMeshMergingDialog;

/** Singleton wrapper to allow for using the setting structure in SSettingsView */
UCLASS(config = Engine)
class UMeshMergingSettingsObject : public UObject
{
	GENERATED_BODY()
public:
	UMeshMergingSettingsObject()		
	{
		Settings.bMergePhysicsData = true;
		// In this case set to AllLODs value since calculating the LODs is not possible and thus disabled in the UI
		Settings.LODSelectionType = EMeshLODSelectionType::AllLODs;
	}

	

	static UMeshMergingSettingsObject* Get()
	{		
		// This is a singleton, duplicate default object
		if (!bInitialized)
		{
			DefaultSettings = DuplicateObject(GetMutableDefault<UMeshMergingSettingsObject>(), nullptr);
			DefaultSettings->AddToRoot();			
			bInitialized = true;
		}

		return DefaultSettings;
	}

	static void Destroy()
	{
		if (bInitialized)
		{
			if (UObjectInitialized() && DefaultSettings)
			{
				DefaultSettings->RemoveFromRoot();
				DefaultSettings->MarkAsGarbage();
			}
			
			DefaultSettings = nullptr;
			bInitialized = false;
		}
	}

protected:
	static UMeshMergingSettingsObject* DefaultSettings;
	static bool bInitialized;

public:
	UPROPERTY(editanywhere, meta = (ShowOnlyInnerProperties), Category = MergeSettings)
	FMeshMergingSettings Settings;
};

/**
 * Mesh Merging Tool
 */
class FMeshMergingTool : public FMergeActorsTool
{
	friend class SMeshMergingDialog;

public:

	FMeshMergingTool();
	~FMeshMergingTool();

	// IMergeActorsTool interface
	virtual TSharedRef<SWidget> GetWidget() override;
	virtual FName GetIconName() const override;
	virtual FText GetToolNameText() const override;
	virtual FText GetTooltipText() const override;
	virtual FString GetDefaultPackageName() const override;

protected:
	virtual bool RunMerge(const FString& PackageName, const TArray<TSharedPtr<FMergeComponentData>>& SelectedComponents) override;	
	virtual const TArray<TSharedPtr<FMergeComponentData>>& GetSelectedComponentsInWidget() const override;

private:
	/** Pointer to the mesh merging dialog containing settings for the merge */
	TSharedPtr<SMeshMergingDialog> MergingDialog;

	/** Pointer to singleton settings object */
	UMeshMergingSettingsObject* SettingsObject;
};
