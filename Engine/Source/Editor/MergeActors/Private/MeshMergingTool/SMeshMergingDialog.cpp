// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshMergingTool/SMeshMergingDialog.h"
#include "MeshMergingTool/MeshMergingTool.h"
#include "SlateOptMacros.h"

#include "IDetailsView.h"

//////////////////////////////////////////////////////////////////////////
// SMeshMergingDialog

SMeshMergingDialog::SMeshMergingDialog()
{
}

SMeshMergingDialog::~SMeshMergingDialog()
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SMeshMergingDialog::Construct(const FArguments& InArgs, FMeshMergingTool* InTool)
{
	checkf(InTool != nullptr, TEXT("Invalid owner tool supplied"));
	Tool = InTool;

	SMeshProxyCommonDialog::Construct(SMeshProxyCommonDialog::FArguments());

	MergeSettings = UMeshMergingSettingsObject::Get();
	SettingsView->SetObject(MergeSettings);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION
