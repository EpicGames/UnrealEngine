// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Components/StaticMeshComponent.h"
#include "Widgets/Views/STableViewBase.h"
#include "Widgets/Views/STableRow.h"

#include "MergeProxyUtils/Utils.h"
#include "MergeProxyUtils/SMeshProxyCommonDialog.h"

class FMeshMergingTool;
class UMeshMergingSettingsObject;

/*-----------------------------------------------------------------------------
   SMeshMergingDialog
-----------------------------------------------------------------------------*/
class SMeshMergingDialog : public SMeshProxyCommonDialog
{
public:
	SLATE_BEGIN_ARGS(SMeshMergingDialog)
	{
	}

	SLATE_END_ARGS()

public:
	/** **/
	SMeshMergingDialog();
	~SMeshMergingDialog();

	/** SWidget functions */
	void Construct(const FArguments& InArgs, FMeshMergingTool* InTool);	

private:
	/** Owning mesh merging tool */
	FMeshMergingTool* Tool;

	/** Cached pointer to mesh merging setting singleton object */
	UMeshMergingSettingsObject* MergeSettings;
};
