// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshProxyTool/MeshProxyTool.h"
#include "Misc/Paths.h"
#include "Misc/FeedbackContext.h"
#include "Modules/ModuleManager.h"
#include "UObject/Package.h"
#include "Misc/PackageName.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "GameFramework/Actor.h"
#include "Components/StaticMeshComponent.h"
#include "Components/InstancedStaticMeshComponent.h"
#include "Components/SplineMeshComponent.h"
#include "Engine/StaticMeshActor.h"
#include "Engine/Selection.h"
#include "Editor.h"
#include "Misc/MessageDialog.h"
#include "MeshProxyTool/SMeshProxyDialog.h"
#include "MeshUtilities.h"
#include "IContentBrowserSingleton.h"
#include "ContentBrowserModule.h"
#include "AssetRegistry/AssetRegistryModule.h"
#include "IMeshReductionInterfaces.h"
#include "IMeshMergeUtilities.h"
#include "MeshMergeModule.h"

#include "MeshProxyTool/SMeshProxyDialog.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MeshProxyTool)

#define LOCTEXT_NAMESPACE "MeshProxyTool"

bool UMeshProxySettingsObject::bInitialized = false;
UMeshProxySettingsObject* UMeshProxySettingsObject::DefaultSettings = nullptr;

FMeshProxyTool::FMeshProxyTool()
{
	SettingsObject = UMeshProxySettingsObject::Get();
}

FMeshProxyTool::~FMeshProxyTool()
{
	UMeshProxySettingsObject::Destroy();
	SettingsObject = nullptr;
}

TSharedRef<SWidget> FMeshProxyTool::GetWidget()
{
	SAssignNew(ProxyDialog, SMeshProxyDialog, this);
	return ProxyDialog.ToSharedRef();
}

FName FMeshProxyTool::GetIconName() const
{
	return "MergeActors.MeshProxyTool";
}

FText FMeshProxyTool::GetToolNameText() const
{
	return LOCTEXT("MeshProxyToolName", "Simplify");
}

FText FMeshProxyTool::GetTooltipText() const
{
	return LOCTEXT("MeshProxyToolTooltip", "Merge source actors meshes and perform a simplification pass. Will generate a single static mesh with baked textures.");
}

FString FMeshProxyTool::GetDefaultPackageName() const
{
	FString PackageName;

	USelection* SelectedActors = GEditor->GetSelectedActors();

	// Iterate through selected actors and find first static mesh asset
	// Use this static mesh path as destination package name for a merged mesh
	for (FSelectionIterator Iter(*SelectedActors); Iter; ++Iter)
	{
		AActor* Actor = Cast<AActor>(*Iter);
		if (Actor)
		{
			TInlineComponentArray<UStaticMeshComponent*> SMComponets;
			Actor->GetComponents(SMComponets);
			for (UStaticMeshComponent* Component : SMComponets)
			{
				if (Component->GetStaticMesh())
				{
					PackageName = FPackageName::GetLongPackagePath(Component->GetStaticMesh()->GetOutermost()->GetName());
					PackageName += FString(TEXT("/PROXY_")) + Component->GetStaticMesh()->GetName();
					break;
				}
			}
		}

		if (!PackageName.IsEmpty())
		{
			break;
		}
	}

	if (PackageName.IsEmpty())
	{
		PackageName = FPackageName::FilenameToLongPackageName(FPaths::ProjectContentDir() + TEXT("PROXY"));
		PackageName = MakeUniqueObjectName(NULL, UPackage::StaticClass(), *PackageName).ToString();
	}

	return PackageName;
}

const TArray<TSharedPtr<FMergeComponentData>>& FMeshProxyTool::GetSelectedComponentsInWidget() const
{
	return ProxyDialog->GetSelectedComponents();
}

void ReplaceSourceActorsByProxyMesh(TArray<UObject*>& NewAssetsToSync, ULevel* Level, TArray<AActor*>& Actors)
{
	UStaticMesh* MergedMesh = nullptr;
	if (NewAssetsToSync.FindItemByClass(&MergedMesh))
	{
		Level->Modify();

		UWorld* World = Level->OwningWorld;
		FActorSpawnParameters Params;
		Params.OverrideLevel = Level;
		FRotator MergedActorRotation(ForceInit);
		// The pivot of the merged mesh is always at the origin
		FVector MergedActorLocation(0, 0, 0);

		AStaticMeshActor* MergedActor = World->SpawnActor<AStaticMeshActor>(MergedActorLocation, MergedActorRotation, Params);
		MergedActor->GetStaticMeshComponent()->SetStaticMesh(MergedMesh);
		MergedActor->SetActorLabel(MergedMesh->GetName());
		World->UpdateCullDistanceVolumes(MergedActor, MergedActor->GetStaticMeshComponent());
		GEditor->SelectNone(true, true);
		GEditor->SelectActor(MergedActor, true, true);

		// Remove source actors
		for (AActor* Actor : Actors)
		{
			Actor->Destroy();
		}
	}
}

bool FMeshProxyTool::RunMerge(const FString& PackageName, const TArray<TSharedPtr<FMergeComponentData>>& SelectedComponents)
{
	TArray<AActor*> Actors;
	TArray<ULevel*> UniqueLevels;
	TArray<UObject*> AssetsToSync;

	BuildActorsListFromMergeComponentsData(SelectedComponents, Actors, bReplaceSourceActors ? &UniqueLevels : nullptr);

	// This restriction is only for replacement of selected actors with merged mesh actor
	if (UniqueLevels.Num() > 1 && bReplaceSourceActors)
	{
		FText Message = NSLOCTEXT("UnrealEd", "FailedToMergeActorsSublevels_Msg", "The selected actors should be in the same level");
		const FText Title = NSLOCTEXT("UnrealEd", "FailedToMergeActors_Title", "Unable to merge actors");
		FMessageDialog::Open(EAppMsgType::Ok, Message, Title);
		return false;
	}

	// Get the module for the mesh merge utilities
	const IMeshMergeUtilities& MeshMergeUtilities = FModuleManager::Get().LoadModuleChecked<IMeshMergeModule>("MeshMergeUtilities").GetUtilities();

	if (Actors.Num())
	{
		GWarn->BeginSlowTask(LOCTEXT("MeshProxy_CreatingProxy", "Creating Mesh Proxy"), true);
		GEditor->BeginTransaction(LOCTEXT("MeshProxy_Create", "Creating Mesh Proxy"));

		FVector ProxyLocation = FVector::ZeroVector;
		TArray<UObject*> NewAssetsToSync;

		FCreateProxyDelegate ProxyDelegate;
		ProxyDelegate.BindLambda(
			[&NewAssetsToSync](const FGuid Guid, TArray<UObject*>& InAssetsToSync)
		{
			//Update the asset registry that a new static mash and material has been created
			if (InAssetsToSync.Num())
			{
				FAssetRegistryModule& AssetRegistry = FModuleManager::Get().LoadModuleChecked<FAssetRegistryModule>("AssetRegistry");
				int32 AssetCount = InAssetsToSync.Num();
				for (int32 AssetIndex = 0; AssetIndex < AssetCount; AssetIndex++)
				{
					AssetRegistry.AssetCreated(InAssetsToSync[AssetIndex]);
					GEditor->BroadcastObjectReimported(InAssetsToSync[AssetIndex]);
				}

				//Also notify the content browser that the new assets exists
				FContentBrowserModule& ContentBrowserModule = FModuleManager::Get().LoadModuleChecked<FContentBrowserModule>("ContentBrowser");
				ContentBrowserModule.Get().SyncBrowserToAssets(InAssetsToSync, true);

				NewAssetsToSync += InAssetsToSync;
			}
		});

		// Extracting static mesh components from the selected mesh components in the dialog
		TArray<UStaticMeshComponent*> StaticMeshComponentsToMerge;

		for (const TSharedPtr<FMergeComponentData>& SelectedComponent : SelectedComponents)
		{
			// Determine whether or not this component should be incorporated according the user settings
			if (SelectedComponent->bShouldIncorporate && SelectedComponent->PrimComponent.IsValid())
			{
				if (UStaticMeshComponent* StaticMeshComponent = Cast<UStaticMeshComponent>(SelectedComponent->PrimComponent.Get()))
					StaticMeshComponentsToMerge.Add(StaticMeshComponent);
			}
		}
		StaticMeshComponentsToMerge.RemoveAll([](UStaticMeshComponent* Val) { return Val->GetStaticMesh() == nullptr; });
		
		if ( StaticMeshComponentsToMerge.Num())
		{
			FGuid JobGuid = FGuid::NewGuid();
			MeshMergeUtilities.CreateProxyMesh(StaticMeshComponentsToMerge, SettingsObject->Settings, nullptr, PackageName, JobGuid, ProxyDelegate);
		}

		if(bReplaceSourceActors && UniqueLevels[0])
		{
			ReplaceSourceActorsByProxyMesh(NewAssetsToSync, UniqueLevels[0], Actors);
		}

		GEditor->EndTransaction();
		GWarn->EndSlowTask();
	}

	return true;
}


#undef LOCTEXT_NAMESPACE

