// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "MeshMerge/MeshProxySettings.h"
#include "MergeActorsTool.h"

#include "MeshProxyTool.generated.h"

class SMeshProxyDialog;

/** Singleton wrapper to allow for using the setting structure in SSettingsView */
UCLASS(config = Engine)
class UMeshProxySettingsObject : public UObject
{
	GENERATED_BODY()
public:
	UMeshProxySettingsObject()
	{
	}

	static UMeshProxySettingsObject* Get()
	{	
		// This is a singleton, duplicate default object
		if (!bInitialized)
		{
			DefaultSettings = DuplicateObject(GetMutableDefault<UMeshProxySettingsObject>(), nullptr);
			DefaultSettings->AddToRoot();
			bInitialized = true;
		}

		return DefaultSettings;
	}

	static void Destroy()
	{
		if (bInitialized)
		{
			if (UObjectInitialized() && DefaultSettings)
			{
				DefaultSettings->RemoveFromRoot();
				DefaultSettings->MarkAsGarbage();
			}

			DefaultSettings = nullptr;
			bInitialized = false;
		}
	}

protected:
	static bool bInitialized;
	static UMeshProxySettingsObject* DefaultSettings;
public:
	UPROPERTY(editanywhere, meta = (ShowOnlyInnerProperties), Category = ProxySettings)
	FMeshProxySettings  Settings;
};

/**
* Mesh Proxy Tool
*/
class FMeshProxyTool : public FMergeActorsTool
{
	friend class SMeshProxyDialog;

public:
	FMeshProxyTool();
	~FMeshProxyTool();

	// IMergeActorsTool interface
	virtual TSharedRef<SWidget> GetWidget() override;
	virtual FName GetIconName() const override;
	virtual FText GetToolNameText() const override;
	virtual FText GetTooltipText() const override;
	virtual FString GetDefaultPackageName() const override;

protected:
	virtual bool RunMerge(const FString& PackageName, const TArray<TSharedPtr<FMergeComponentData>>& SelectedComponents) override;
	virtual const TArray<TSharedPtr<FMergeComponentData>>& GetSelectedComponentsInWidget() const override;

protected:

	/** Pointer to the mesh merging dialog containing settings for the merge */
	TSharedPtr<SMeshProxyDialog> ProxyDialog;

	/** Pointer to singleton settings object */
	UMeshProxySettingsObject* SettingsObject;
};

