// Copyright Epic Games, Inc. All Rights Reserved.

#include "MeshProxyTool/SMeshProxyDialog.h"
#include "Editor.h"
#include "Styling/AppStyle.h"
#include "Engine/Selection.h"
#include "MeshMerge/MeshProxySettings.h"
#include "MeshProxyTool/MeshProxyTool.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "Styling/SlateTypes.h"
#include "SlateOptMacros.h"
#include "UObject/UnrealType.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/SNumericEntryBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"
#include "Widgets/Views/SListView.h"


#define LOCTEXT_NAMESPACE "SMeshProxyDialog"

//////////////////////////////////////////////////////////////////////////
// SMeshProxyDialog
SMeshProxyDialog::SMeshProxyDialog()
{
    MergeStaticMeshComponentsLabel = LOCTEXT("CreateProxyMeshComponentsLabel", "Mesh components used to compute the proxy mesh:");
	SelectedComponentsListBoxToolTip = LOCTEXT("CreateProxyMeshSelectedComponentsListBoxToolTip", "The selected mesh components will be used to compute the proxy mesh");
    DeleteUndoLabel = LOCTEXT("DeleteUndo", "Insufficient mesh components found for ProxyLOD merging.");
}

SMeshProxyDialog::~SMeshProxyDialog()
{
}

BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void  SMeshProxyDialog::Construct(const FArguments& InArgs, FMeshProxyTool* InTool)
{
	checkf(InTool != nullptr, TEXT("Invalid owner tool supplied"));
	Tool = InTool;

	SMeshProxyCommonDialog::Construct(SMeshProxyCommonDialog::FArguments());

	ProxySettings = UMeshProxySettingsObject::Get();
	SettingsView->SetObject(ProxySettings);
}

END_SLATE_FUNCTION_BUILD_OPTIMIZATION


#undef LOCTEXT_NAMESPACE
