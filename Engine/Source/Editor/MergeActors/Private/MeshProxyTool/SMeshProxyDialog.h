// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/Views/STableViewBase.h"

#include "MergeProxyUtils/Utils.h"
#include "MergeProxyUtils/SMeshProxyCommonDialog.h"

class FMeshProxyTool;
class UMeshProxySettingsObject;
class UObject;

/*-----------------------------------------------------------------------------
SMeshProxyDialog  
-----------------------------------------------------------------------------*/

class SMeshProxyDialog : public SMeshProxyCommonDialog
{
public:
	SLATE_BEGIN_ARGS(SMeshProxyDialog)
	{
	}
	SLATE_END_ARGS()

public:
	/** **/
	SMeshProxyDialog();
	~SMeshProxyDialog();

	/** SWidget functions */
	void Construct(const FArguments& InArgs, FMeshProxyTool* InTool);

private:
	/** Owning mesh merging tool */
	FMeshProxyTool* Tool;

	/** Cached pointer to mesh merging setting singleton object */
	UMeshProxySettingsObject* ProxySettings;
};

