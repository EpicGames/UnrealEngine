// Copyright Epic Games, Inc. All Rights Reserved.

#include "SMergeActorsToolbar.h"
#include "IMergeActorsTool.h"
#include "Modules/ModuleManager.h"
#include "Widgets/SBoxPanel.h"
#include "Styling/SlateTypes.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SScrollBox.h"
#include "Widgets/Input/SCheckBox.h"
#include "Widgets/Input/STextComboBox.h"
#include "Styling/AppStyle.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#include "LevelEditor.h"
#include "IDocumentation.h"
#include "Styling/ToolBarStyle.h"
#include "SPrimaryButton.h"

#define LOCTEXT_NAMESPACE "SMergeActorsToolbar"


//////////////////////////////////////////////////////////////

void SMergeActorsToolbar::Construct(const FArguments& InArgs)
{
	// Important: We use raw bindings here because we are releasing our binding in our destructor (where a weak pointer would be invalid)
	// It's imperative that our delegate is removed in the destructor for the level editor module to play nicely with reloading.

	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.OnActorSelectionChanged().AddRaw(this, &SMergeActorsToolbar::OnActorSelectionChanged);

	RegisteredTools = InArgs._ToolsToRegister;

	ChildSlot
	[
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		.Padding(0, 0, 0, 0)
		[
			SAssignNew(ToolbarContainer, SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(10)
		]

		+ SVerticalBox::Slot()
		.FillHeight(1.0f)
		.Padding(0, 0, 0, 0)
		[
			SNew(SBorder)
			.BorderImage(FStyleDefaults::GetNoBrush())
			.Padding(0.f)
			.IsEnabled(this, &SMergeActorsToolbar::GetContentEnabledState)
			[
				SNew(SScrollBox)
				+SScrollBox::Slot()
				[
					SAssignNew(InlineContentHolder, SBox)
				]
			]
		]

		+ SVerticalBox::Slot()
		.AutoHeight()
		.VAlign(VAlign_Bottom)
		[
			SNew(SBorder)
			.BorderImage(FAppStyle::GetBrush("ToolPanel.GroupBorder"))
			.Padding(10)
			[
				SNew(SHorizontalBox) 

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Left)
				.Padding(FAppStyle::GetMargin("StandardDialog.ContentPadding"))
				[
					SNew(SCheckBox)
					.Type(ESlateCheckBoxType::CheckBox)
					.IsChecked_Lambda([this]() { return this->RegisteredTools[CurrentlySelectedTool]->GetReplaceSourceActors() ? ECheckBoxState::Checked : ECheckBoxState::Unchecked; })
					.OnCheckStateChanged_Lambda([this](ECheckBoxState NewValue) { this->RegisteredTools[CurrentlySelectedTool]->SetReplaceSourceActors(NewValue == ECheckBoxState::Checked); })
					.Content()
					[
						SNew(STextBlock)
						.Text(LOCTEXT("ReplaceSourceActorsLabel", "Replace Source Actors"))
						.Font(FAppStyle::GetFontStyle("StandardDialog.SmallFont"))
					]
				]

				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Right)
				.Padding(4, 4, 10, 4)
				[
					SNew(SPrimaryButton)
					.Text(LOCTEXT("MergeActors", "Merge Actors"))
					.OnClicked(this, &SMergeActorsToolbar::OnMergeActorsClicked)
					.IsEnabled_Lambda([this]() -> bool { return this->RegisteredTools[CurrentlySelectedTool]->CanMergeFromWidget(); })
				]
			]
		]
	];

	UpdateToolbar();

	// Update selected actor state for the first time
	GUnrealEd->UpdateFloatingPropertyWindows();
}


SMergeActorsToolbar::~SMergeActorsToolbar()
{
	FLevelEditorModule& LevelEditor = FModuleManager::GetModuleChecked<FLevelEditorModule>("LevelEditor");
	LevelEditor.OnActorSelectionChanged().RemoveAll(this);
}


void SMergeActorsToolbar::OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh)
{
	SelectedObjects = NewSelection;
	bIsContentEnabled = (NewSelection.Num() > 0);
}

void SMergeActorsToolbar::OnToolSelectionChanged(TSharedPtr<FDropDownItem> NewSelection, ESelectInfo::Type SelectInfo)
{
	int32 Index = 0;
	if (ToolDropDownEntries.Find(NewSelection, Index) && Index != CurrentlySelectedTool)
	{
		CurrentlySelectedTool = Index;
		UpdateInlineContent();
	}
}

FReply SMergeActorsToolbar::OnMergeActorsClicked()
{
	if (CurrentlySelectedTool >= 0 && CurrentlySelectedTool < RegisteredTools.Num())
	{
		RegisteredTools[CurrentlySelectedTool]->RunMergeFromWidget();
	}

	return FReply::Handled();
}


bool SMergeActorsToolbar::GetContentEnabledState() const
{
	return bIsContentEnabled;
}


void SMergeActorsToolbar::AddTool(IMergeActorsTool* Tool)
{
	check(!RegisteredTools.Contains(Tool));
	RegisteredTools.Add(Tool);
	UpdateToolbar();
}


void SMergeActorsToolbar::RemoveTool(IMergeActorsTool* Tool)
{
	int32 IndexToRemove = RegisteredTools.Find(Tool);
	if (IndexToRemove != INDEX_NONE)
	{
		RegisteredTools.RemoveAt(IndexToRemove);

		if (CurrentlySelectedTool > IndexToRemove)
		{
			CurrentlySelectedTool--;
		}
		UpdateToolbar();
	}
}

SMergeActorsToolbar::FDropDownItem::FDropDownItem(const FText& InName, const FName& InIconName, const FText& InDescription)
	: Name(InName)
	, IconName(InIconName)
	, Description(InDescription)
{}

void SMergeActorsToolbar::UpdateToolbar()
{
	// Build tools entries data for the drop-down
	ToolDropDownEntries.Empty();

	for(int32 ToolIndex = 0; ToolIndex < RegisteredTools.Num(); ++ToolIndex)
	{
		const IMergeActorsTool* Tool = RegisteredTools[ToolIndex];
		ToolDropDownEntries.Add(MakeShareable(new FDropDownItem(Tool->GetToolNameText(), Tool->GetIconName(), Tool->GetTooltipText())));
	}

	// Build combo box
	const ISlateStyle& StyleSet = FAppStyle::Get();

	TSharedRef <SComboBox<TSharedPtr<FDropDownItem> > > ComboBox =
		SNew(SComboBox<TSharedPtr<FDropDownItem> >)
		.OptionsSource(&ToolDropDownEntries)
		.OnGenerateWidget(this, &SMergeActorsToolbar::MakeWidgetFromEntry)
		.InitiallySelectedItem(ToolDropDownEntries[CurrentlySelectedTool])
		.OnSelectionChanged(this, &SMergeActorsToolbar::OnToolSelectionChanged)
		.Content()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image_Lambda([this](){
					return FAppStyle::Get().GetBrush(ToolDropDownEntries[CurrentlySelectedTool]->IconName);
				})
			]
			+ SHorizontalBox::Slot()
			.FillWidth(1)
			.VAlign(VAlign_Center)
			.Padding(5.0, 0, 0, 0)
			[
				SNew(STextBlock)
				.Text_Lambda([this]() -> FText {
					return ToolDropDownEntries[CurrentlySelectedTool]->Name;
				})
			]
		];

	TSharedRef<SVerticalBox> ToolbarContent =
		SNew(SVerticalBox)

		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(HAlign_Left)
			.VAlign(VAlign_Center)
			.AutoWidth()
			[
				SNew(STextBlock)
				.Text(LOCTEXT("MergeMethodLabel", "Merge Method"))
			]
			+ SHorizontalBox::Slot()
			.VAlign(VAlign_Center)
			.Padding(10.0, 0, 0, 0)
			[
				ComboBox
			]
			// Filler so that the combo box is just the right size
			+ SHorizontalBox::Slot()
			[
				SNew(SBorder)
				.BorderImage(FStyleDefaults::GetNoBrush())
			]
		]
		+ SVerticalBox::Slot()
		.Padding(10, 10, 10, 10)
		.AutoHeight()
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.AutoWidth()
			.HAlign(HAlign_Center)
			.VAlign(VAlign_Center)
			[
				SNew(SImage)
				.Image(FAppStyle::Get().GetBrush("Icons.Info"))
			]
			+ SHorizontalBox::Slot()
			.Padding(10, 0, 0, 0)
			[
				SNew(STextBlock)
				.AutoWrapText(true)
				.Text_Lambda([this]() -> FText {
					return ToolDropDownEntries[CurrentlySelectedTool]->Description;
				})
			]
		];

	ToolbarContainer->SetContent(ToolbarContent);

	UpdateInlineContent();
}

TSharedRef<SWidget> SMergeActorsToolbar::MakeWidgetFromEntry(TSharedPtr<FDropDownItem> InItem)
{
	return SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.VAlign(VAlign_Center)
		[
			SNew(SImage)
			.Image(FAppStyle::Get().GetBrush(InItem->IconName))
		]
		+ SHorizontalBox::Slot()
		.FillWidth(1)
		.VAlign(VAlign_Center)
		.Padding(5.0, 0, 0, 0)
		[
			SNew(STextBlock)
			.Text(InItem->Name)
		];
}

void SMergeActorsToolbar::UpdateInlineContent()
{
	if (CurrentlySelectedTool >= 0 && CurrentlySelectedTool < RegisteredTools.Num())
	{
		InlineContentHolder->SetContent(RegisteredTools[CurrentlySelectedTool]->GetWidget());
	}
}


#undef LOCTEXT_NAMESPACE
