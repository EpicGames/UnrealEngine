// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Input/Reply.h"
#include "Widgets/SCompoundWidget.h"

class IMergeActorsTool;
class SBorder;
class SBox;
enum class ECheckBoxState : uint8;

//////////////////////////////////////////////////////////////////////////
// SMergeActorsToolbar

class SMergeActorsToolbar : public SCompoundWidget
{
public:

	SLATE_BEGIN_ARGS(SMergeActorsToolbar) {}
		SLATE_ARGUMENT(TArray<IMergeActorsTool*>, ToolsToRegister)
	SLATE_END_ARGS()

	/**
	 * Construct the widget
	 *
	 * @param	InArgs			A declaration from which to construct the widget
	 */
	void Construct(const FArguments& InArgs);

	/** Constructor */
	SMergeActorsToolbar()
		: CurrentlySelectedTool(0)
	{}

	/** Destructor */
	virtual ~SMergeActorsToolbar();

	/** Add a new tool to the toolbar */
	void AddTool(IMergeActorsTool* Tool);

	/** Remove an existing tool from the toolbar */
	void RemoveTool(IMergeActorsTool* Tool);


private:
	struct FDropDownItem
	{
		FDropDownItem(const FText& InName, const FName& InIconName, const FText& InDescription);

		FText Name;
		FName IconName;
		FText Description;
	};

	/** Called to create the combo box entries */
	TSharedRef<SWidget> MakeWidgetFromEntry(TSharedPtr<FDropDownItem> InItem);

	/** Called when the level actor selection changes */
	void OnActorSelectionChanged(const TArray<UObject*>& NewSelection, bool bForceRefresh);

	/** Called when the currently selected tool changes */
	void OnToolSelectionChanged(TSharedPtr<FDropDownItem> NewSelection, ESelectInfo::Type SelectInfo);

	/** Called when the Merge Actors button is clicked */
	FReply OnMergeActorsClicked();

	/** Determine whether the widget content is enabled or not */
	bool GetContentEnabledState() const;

	/** Update the toolbar container based on the currently registered tools */
	void UpdateToolbar();

	/** Updates the inline content widget for the current tool */
	void UpdateInlineContent();

private:

	/** List of registered tool instances */
	TArray<IMergeActorsTool*> RegisteredTools;

	/** Index of currently selected tool */
	int32 CurrentlySelectedTool;

	/** List of currently selected objects */
	TArray<UObject*> SelectedObjects;

	/** Whether the merge actors tool panel is enabled or not */
	bool bIsContentEnabled;

	/** List of register tool names */
	TArray< TSharedPtr<FDropDownItem> > ToolDropDownEntries;

	/** The container holding the toolbar */
	TSharedPtr<SBorder> ToolbarContainer;

	/** Inline content area for different tool modes */
	TSharedPtr<SBox> InlineContentHolder;
};
