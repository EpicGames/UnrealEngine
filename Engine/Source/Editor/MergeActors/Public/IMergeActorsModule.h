// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

class IMergeActorsTool;

/**
 * Merge Actors module interface
 */
class IMergeActorsModule : public IModuleInterface
{

public:

	/**
	 * Get reference to the Merge Actors module instance
	 */
	static inline IMergeActorsModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IMergeActorsModule>("MergeActors");
	}

	/**
	 * Register an IMergeActorsTool with the module, passing ownership to it
	 */
	virtual bool RegisterMergeActorsTool(TUniquePtr<IMergeActorsTool> Tool) = 0;

	/**
	 * Unregister an IMergeActorsTool with the module
	 */
	virtual bool UnregisterMergeActorsTool(IMergeActorsTool* Tool) = 0;

	/**
	 * Get currently registered list of MergeActors tools
	 */
	virtual void GetRegisteredMergeActorsTools(TArray<IMergeActorsTool*>& OutTools) = 0;
};
