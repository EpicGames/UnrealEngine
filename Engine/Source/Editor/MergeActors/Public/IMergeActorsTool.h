// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"

/**
* Merge Actors tool interface
*/
class IMergeActorsTool
{

public:

	/** Virtual destructor */
	virtual ~IMergeActorsTool() = default;

	/**
	 * Gets the widget instance associated with this tool
	 */
	virtual TSharedRef<SWidget> GetWidget() = 0;

	/**
	 * Get the name of the icon displayed in the Merge Actors toolbar
	 */
	virtual FName GetIconName() const = 0;

	/**
	 * Get tool name text to be displayed in the menus & Merge Actors toolbar
	 */
	virtual FText GetToolNameText() const = 0;

	/**
	 * Get Tooltip text displayed in the Merge Actors toolbar
	 */
	virtual FText GetTooltipText() const = 0;

	/**
	 * Get default name for the merged asset package
	 */
	virtual FString GetDefaultPackageName() const = 0;

	/**
	 * Checks if the Replace Source Actors option is selected
	 */
	virtual bool GetReplaceSourceActors() const = 0;

	/**
	 * Changes the Replace Source Actors option
	 * 
	 * @param whether to replace the source actors or not
	 */
	virtual void SetReplaceSourceActors(bool bReplaceSourceActors) = 0;

	/**
	 * Perform merge operation from the current selection
	 *
	 * @return true if the merge succeeded
	 */
	virtual bool RunMergeFromSelection() = 0;

	/**
	 * Perform merge operation from the selection in the Merge Actors panel
	 */
	virtual bool RunMergeFromWidget() = 0;
	
	/*
	* Checks if merge operation is valid from the current selection
	*
	* @return true if merge can be executed
	*/
	virtual bool CanMergeFromSelection() const = 0;

	/*
	* Checks if merge operation is valid from the current selection
	*
	* @return true if merge can be executed
	*/
	virtual bool CanMergeFromWidget() const = 0;
};
